[WorldTypeTraceFilter]
DisabledWorldTypes=2+4