// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataSourceFiltering.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataSourceFiltering)

IMPLEMENT_MODULE(FDefaultModuleImpl, SourceFilteringCore);


bool operator== (const FActorClassFilter& LHS, const FActorClassFilter& RHS)
{
	return LHS.ActorClass == RHS.ActorClass;
}
