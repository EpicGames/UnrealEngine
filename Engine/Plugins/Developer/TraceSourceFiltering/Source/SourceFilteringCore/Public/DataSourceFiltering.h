// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/EnumRange.h"

#include "DataSourceFiltering.generated.h"

/** Enum representing the possible Filter Set operations */
UENUM()
enum class EFilterSetMode : uint8
{
	AND,
	OR,
	NOT,
	Count UMETA(Hidden)
};

ENUM_RANGE_BY_COUNT(EFilterSetMode, EFilterSetMode::Count);

/** Enum representing all possible Source Filter operations, used by FSourceFilterTrace::OutputFilterOperation */
enum class ESourceActorFilterOperation
{
	RemoveFilter,
	MoveFilter,
	ReplaceFilter,
	SetFilterMode,
	SetFilterState
};

/** Enum representing all possible World Filter operations, used by FSourceFilterTrace::OutputWorldOperation */
enum class EWorldFilterOperation
{
	TypeFilter,
	NetModeFilter,
	InstanceFilter,
	RemoveWorld
};

/** High-level filter structure used when filtering AActor instances to apply user filters to inside of a UWorld */
USTRUCT()
struct FActorClassFilter
{
	GENERATED_BODY()
public:

	/** Target actor class used when applying this filter */
	UPROPERTY(EditAnywhere, Category = TraceSourceFiltering)
	FSoftClassPath ActorClass;

	/** Flag as to whether or not any derived classes from ActorClass should also be considered when filtering */
	UPROPERTY(EditAnywhere, Category = TraceSourceFiltering)
	bool bIncludeDerivedClasses = false;

	inline bool operator==(const FActorClassFilter Rhs)
	{
		return ActorClass == Rhs.ActorClass;
	}

	SOURCEFILTERINGCORE_API friend bool operator==(const FActorClassFilter& LHS, const FActorClassFilter& RHS);
};
