// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "IDataSourceFilterInterface.generated.h"

#define UE_API SOURCEFILTERINGCORE_API

USTRUCT(BlueprintType)
struct FDataSourceFilterConfiguration
{
	GENERATED_BODY()

	FDataSourceFilterConfiguration() : bOnlyApplyDuringActorSpawn(false), bCanRunAsynchronously(false), FilterApplyingTickInterval(1) {}

	/** Flag whether or not this filter should only applied once, whenever an actor is spawned */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Filtering)
	uint8 bOnlyApplyDuringActorSpawn : 1;

	/** Flag whether or not this filter does not rely on gamethread only data, and therefore can work on a differen thread */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = Filtering)
	uint8 bCanRunAsynchronously : 1;

	/** Interval, in frames, between applying the filter. The resulting value is cached for intermediate frames. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Filtering, meta=(EditCondition="!bOnlyApplyDuringActorSpawn", ClampMin="1", ClampMax="255", UIMin="1", UIMax="128"))
	uint8 FilterApplyingTickInterval;
};

UINTERFACE(MinimalAPI, Blueprintable)
class UDataSourceFilterInterface : public UInterface
{
	GENERATED_BODY()
};

/** Interface used for implementing Engine and UnrealInsights versions respectively UDataSourceFilter and UTraceDataSourceFilter */
class IDataSourceFilterInterface
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintNativeEvent, Category = TraceSourceFiltering)
	UE_API void GetDisplayText(FText& OutDisplayText) const;
	virtual void GetDisplayText_Implementation(FText& OutDisplayText) const
	{
		return GetDisplayText_Internal(OutDisplayText);
	}

	UFUNCTION(BlueprintNativeEvent, Category = TraceSourceFiltering)
	UE_API void GetToolTipText(FText& OutDisplayText) const;
	virtual void GetToolTipText_Implementation(FText& OutDisplayText) const
	{
		return GetToolTipText_Internal(OutDisplayText);
	}

	virtual const FDataSourceFilterConfiguration& GetConfiguration() const = 0;
	virtual void SetEnabled(bool bState) = 0;
	virtual bool IsEnabled() const = 0;
protected:
	virtual void GetDisplayText_Internal(FText& OutDisplayText) const { OutDisplayText = NSLOCTEXT("IDataSourceFilterInterface", "GetDisplayTextNotImplementedText", "IDataSourceFilterInterface::GetDisplayText_Internal not overidden (missing implementation)"); }
	virtual void GetToolTipText_Internal(FText& OutDisplayText) const { GetDisplayText_Internal(OutDisplayText); }
};

#undef UE_API
