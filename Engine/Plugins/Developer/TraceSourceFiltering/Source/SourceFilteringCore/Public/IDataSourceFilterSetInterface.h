// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "IDataSourceFilterSetInterface.generated.h"

enum class EFilterSetMode : uint8;

UINTERFACE(MinimalAPI, Blueprintable)
class UDataSourceFilterSetInterface : public UInterface
{
	GENERATED_BODY()
};

/** Interface used for implementing Engine and UnrealInsights versions respectively UDataSourceFilterSet and UTraceDataSourceFilterSet */
class IDataSourceFilterSetInterface
{
	GENERATED_BODY()

public:
	/** Return the current Filter Set operation mode */
	virtual EFilterSetMode GetFilterSetMode() const = 0;
};
