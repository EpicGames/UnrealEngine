// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "TraceSourceFilteringSettings.generated.h"

UCLASS(MinimalAPI, Config=TraceSourceFilters)
class UTraceSourceFilteringSettings : public UObject
{
	GENERATED_BODY()

public:
	UTraceSourceFilteringSettings() : bDrawFilteringStates(false), bDrawOnlyPassingActors(false), bDrawFilterDescriptionForRejectedActors(false) {}
	
#if WITH_EDITOR
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override
	{
		OnSourceFilteringSettingsChanged.Broadcast();
	}
#endif // WITH_EDITOR

	FSimpleMulticastDelegate& GetOnSourceFilteringSettingsChanged()
	{
		return OnSourceFilteringSettingsChanged;
	}

	/** Whether or not the filtering state for all considered AActor's inside for a UWorld should be drawn using a wire frame box */
	UPROPERTY(Config)
	bool bDrawFilteringStates;

	/** Whether or not only AActor's that are not filtered out should be considered for drawing their wireframe box */
	UPROPERTY(Config)
	bool bDrawOnlyPassingActors;

	/** Whether or not to draw the failed UDataSourceFilter's description for AActor's that did not pass the filtering */
	UPROPERTY(Config)
	bool bDrawFilterDescriptionForRejectedActors;

	/** Whether or not to, whenever it changes, output the optimized filter setup state to the log */
	UPROPERTY(Config)
	bool bOutputOptimizedFilterState;

protected:
	/** Delegate called whenever the contained properties change (see PostEditChangeProperty) */
	FSimpleMulticastDelegate OnSourceFilteringSettingsChanged;
};
