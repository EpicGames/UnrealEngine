// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Class.h"

class FClassFilterObject : public TSharedFromThis<FClassFilterObject>
{
public:
	FClassFilterObject(UClass* InClass, bool bDerivedClasses) : Class(InClass), bIncludeDerived(bDerivedClasses){}

	const FText GetDisplayText() const 
	{
		return FText::FromString(
			FString::Printf(TEXT("%s%s"),
				bIncludeDerived ? TEXT("Is Child Of ") : TEXT("Is a "),
				*Class->GetName()
			)
		);
	}

	UClass* GetClass() const { return Class; }

	bool IncludesDerivedClasses() const { return bIncludeDerived; }
protected:
	UClass* Class;
	bool bIncludeDerived;
};
