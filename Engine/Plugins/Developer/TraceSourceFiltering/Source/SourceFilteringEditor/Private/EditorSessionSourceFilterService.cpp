// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "EditorSessionSourceFilterService.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ScopedTransaction.h"
#include "Editor.h"
#include "IAssetTools.h"
#include "Misc/TransactionObjectEvent.h"
#include "Factories/BlueprintFactory.h"

#include "SourceFilterCollection.h"
#include "TraceSourceFiltering.h"
#include "FilterObject.h"
#include "FilterSetObject.h"
#include "ClassFilterObject.h"

#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "DataSourceFilterSet.h"

#include "PackageHelperFunctions.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "Misc/FileHelper.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "EditorDirectories.h"
#include "AssetToolsModule.h"
#include "TraceSourceFilteringSettings.h"
#include "TraceWorldFiltering.h"
#include "TreeViewBuilder.h"
#include "WorldObject.h"
#include "Widgets/SBoxPanel.h"
#include "WorldFilters.h"
#include "SourceFilterStyle.h"
#include "TraceFilter.h"
#include "Subsystems/AssetEditorSubsystem.h"

#define LOCTEXT_NAMESPACE "FEditorSourceFilterService"

FString FEditorSessionSourceFilterService::TransactionContext = TEXT("FEditorSourceFilterService");

FEditorSessionSourceFilterService::FEditorSessionSourceFilterService()
{
	FilterCollection = FTraceSourceFiltering::Get().GetFilterCollection();
	FilterCollection->GetSourceFiltersUpdated().AddRaw(this, &FEditorSessionSourceFilterService::StateChanged);

	FCoreUObjectDelegates::OnObjectsReplaced.AddRaw(this, &FEditorSessionSourceFilterService::OnObjectsReplaced);

	// Register delegate to catch engine-level trace filtering system changes 
	FTraceWorldFiltering::OnFilterStateChanged().AddRaw(this, &FEditorSessionSourceFilterService::StateChanged);	
	SetupWorldFilters();
	FEditorDelegates::OnAssetsPreDelete.AddRaw(this, &FEditorSessionSourceFilterService::OnAssetsPendingDelete);
}

FEditorSessionSourceFilterService::~FEditorSessionSourceFilterService()
{
	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
	
	FilterCollection->GetSourceFiltersUpdated().RemoveAll(this);
	FTraceWorldFiltering::OnFilterStateChanged().RemoveAll(this);
	FEditorDelegates::OnAssetsPreDelete.RemoveAll(this);
}

void FEditorSessionSourceFilterService::AddFilter(const FString& FilterClassName)
{
	FSoftClassPath ClassPath(FilterClassName);
	if (UClass* Class = ClassPath.TryLoadClass<UDataSourceFilter>())
	{
		const FScopedTransaction Transaction(*FEditorSessionSourceFilterService::TransactionContext, LOCTEXT("AddFilter", "Add Filter"), FilterCollection);
		FilterCollection->Modify();

		FilterCollection->AddFilterOfClass(Class);
	}
}

void FEditorSessionSourceFilterService::AddFilterToSet(TSharedRef<const IFilterObject> FilterSet, const FString& FilterClassName)
{
	FSoftClassPath ClassPath(FilterClassName);
	if (UClass* Class = ClassPath.TryLoadClass<UDataSourceFilter>())
	{
		const FScopedTransaction Transaction(*FEditorSessionSourceFilterService::TransactionContext, LOCTEXT("AddFilterToSet", "Add Filter to Filter Set"), FilterCollection);
		FilterCollection->Modify();

		FilterCollection->AddFilterOfClassToSet(Class, CastChecked<UDataSourceFilterSet>(FilterSet->GetFilter()));
	}
}

void FEditorSessionSourceFilterService::AddFilterToSet(TSharedRef<const IFilterObject> FilterSet, TSharedRef<const IFilterObject> ExistingFilter)
{
	const FScopedTransaction Transaction(*FEditorSessionSourceFilterService::TransactionContext, LOCTEXT("MoveFilter", "Moving Filter"), FilterCollection);
	FilterCollection->Modify();

	UDataSourceFilter* Filter = CastChecked<UDataSourceFilter>(ExistingFilter->GetFilter());
	UDataSourceFilterSet* Set = CastChecked<UDataSourceFilterSet>(FilterSet->GetFilter());
	FilterCollection->MoveFilter(Filter, Set);
}

void FEditorSessionSourceFilterService::RemoveFilter(TSharedRef<const IFilterObject> InFilter)
{
	const FScopedTransaction Transaction(*FEditorSessionSourceFilterService::TransactionContext, LOCTEXT("RemoveFilter", "Removing Filter"), InFilter->GetFilter());
	FilterCollection->Modify();

	FilterCollection->RemoveFilter(CastChecked<UDataSourceFilter>(InFilter->GetFilter()));
}

void FEditorSessionSourceFilterService::SetFilterSetMode(TSharedRef<const IFilterObject> InFilter, EFilterSetMode Mode)
{
	if (UDataSourceFilterSet* FilterSet = CastChecked<UDataSourceFilterSet>(InFilter->GetFilter()))
	{
		const FScopedTransaction Transaction(*FEditorSessionSourceFilterService::TransactionContext, LOCTEXT("SetFilterSetMode", "Set Filter Set Mode"), FilterSet);

		FilterCollection->SetFilterSetMode(FilterSet, Mode);
	}
}

void FEditorSessionSourceFilterService::SetFilterState(TSharedRef<const IFilterObject> InFilter, bool bState)
{
	if (UDataSourceFilter* Filter = CastChecked<UDataSourceFilter>(InFilter->GetFilter()))
	{
		const FScopedTransaction Transaction(*FEditorSessionSourceFilterService::TransactionContext, LOCTEXT("SetFilterState", "Set Filter State"), Filter);

		FilterCollection->SetFilterState(Filter, bState);
	}
}

void FEditorSessionSourceFilterService::ResetFilters()
{
	const FScopedTransaction Transaction(*FEditorSessionSourceFilterService::TransactionContext, LOCTEXT("ResetFilters", "Reset Filters"), FilterCollection);
	FilterCollection->Modify();

	FilterCollection->Reset();
}

void FEditorSessionSourceFilterService::StateChanged()
{
	GetOnSessionStateChanged().Broadcast();
}

void FEditorSessionSourceFilterService::UpdateFilterSettings(UTraceSourceFilteringSettings* InSettings)
{
	InSettings->PostEditChange();
}

UTraceSourceFilteringSettings* FEditorSessionSourceFilterService::GetFilterSettings()
{
	return FTraceSourceFiltering::Get().GetSettings();
}

bool FEditorSessionSourceFilterService::IsActionPending() const
{
	return false;
}

TSharedRef<SWidget> FEditorSessionSourceFilterService::GetFilterPickerWidget(FOnFilterClassPicked InFilterClassPicked)
{
	/** Class filter implementation, ensuring we only show valid UDataSourceFilter (sub)classes */
	class FFilterClassFilter : public IClassViewerFilter
	{
	public:
		bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
		{
			if (InClass->HasAnyClassFlags(CLASS_Abstract | CLASS_HideDropDown | CLASS_Deprecated))
			{
				return false;
			}

			return InClass->IsChildOf(UDataSourceFilter::StaticClass()) && InClass != UDataSourceFilter::StaticClass() && InClass != UDataSourceFilterSet::StaticClass();
		}

		virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
		{
			return InClass->IsChildOf(UDataSourceFilter::StaticClass());
		}
	};

	FClassViewerInitializationOptions Options;
	Options.bShowUnloadedBlueprints = true;
	Options.bShowNoneOption = false;
	TSharedPtr<FFilterClassFilter> ClassFilter = MakeShareable(new FFilterClassFilter);
	Options.ClassFilters.Add(ClassFilter.ToSharedRef());

	FOnClassPicked ClassPicked = FOnClassPicked::CreateLambda([InFilterClassPicked](UClass* Class)
	{
		InFilterClassPicked.ExecuteIfBound(Class->GetPathName());
	});

	return SNew(SBox)
		.WidthOverride(280)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.MaxHeight(500)
			[
				FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer").CreateClassViewer(Options, ClassPicked)
			]
		];
}

TSharedRef<SWidget> FEditorSessionSourceFilterService::GetClassFilterPickerWidget(FOnFilterClassPicked InFilterClassPicked)
{
	/** Class filter implementation, ensuring we only show valid UDataSourceFilter (sub)classes */
	class FFilterClassFilter : public IClassViewerFilter
	{
	public:
		FFilterClassFilter(const TArray<TSharedPtr<FClassFilterObject>>& InExistingClasses) : ExistingClasses(InExistingClasses) {}

		bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
		{
			if (InClass->HasAnyClassFlags(CLASS_Abstract | CLASS_HideDropDown | CLASS_Deprecated))
			{
				return false;
			}

			if (ExistingClasses.ContainsByPredicate([InClass](TSharedPtr<FClassFilterObject> Object)
			{
				return Object->GetClass() == InClass;
			}))
			{
				return false;
			}

			return InClass->IsChildOf(AActor::StaticClass());
		}

		virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
		{
			return InClass->IsChildOf(AActor::StaticClass());
		}

	protected:
		const TArray<TSharedPtr<FClassFilterObject>> ExistingClasses;
	};

	FClassViewerInitializationOptions Options;
	Options.bShowUnloadedBlueprints = true;
	Options.bShowNoneOption = false;

	TArray<TSharedPtr<FClassFilterObject>> ExistingClasses;
	GetClassFilters(ExistingClasses);

	TSharedPtr<FFilterClassFilter> ClassFilter = MakeShared<FFilterClassFilter>(ExistingClasses);
	Options.ClassFilters.Add(ClassFilter.ToSharedRef());

	FOnClassPicked ClassPicked = FOnClassPicked::CreateLambda([InFilterClassPicked](UClass* Class)
	{
		InFilterClassPicked.ExecuteIfBound(Class->GetPathName());
	});

	return SNew(SBox)
		.WidthOverride(280)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.MaxHeight(500)
			[
				FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer").CreateClassViewer(Options, ClassPicked)
			]
		];
}

/** Helper function to have user pick a (existing) package for saving a filter preset */
static bool GetSavePresetPackageName(FString& OutPackageName)
{
	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");

	const FString DefaultName = LOCTEXT("NewFilterPreset", "NewFilterPreset").ToString();	
	const FString DefaultPath = TEXT("/TraceSourceFilters/");

	FString UniqueAssetName;
	FString UniquePackageName;
	AssetToolsModule.Get().CreateUniqueAssetName(DefaultPath / DefaultName, TEXT(""), UniquePackageName, UniqueAssetName);	

	FSaveAssetDialogConfig SaveAssetDialogConfig;
	{
		SaveAssetDialogConfig.DefaultPath = DefaultPath;
		SaveAssetDialogConfig.DefaultAssetName = FPaths::GetCleanFilename(UniqueAssetName);
		SaveAssetDialogConfig.AssetClassNames.Add(USourceFilterCollection::StaticClass()->GetClassPathName());
		SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::AllowButWarn;
		SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("SaveSourceFilterPresetDialogTitle", "Save Filter Preset");
	}

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	FString SaveObjectPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);

	if (!SaveObjectPath.IsEmpty())
	{
		OutPackageName = FPackageName::ObjectPathToPackageName(SaveObjectPath);

		FText OutError;
		return FFileHelper::IsFilenameValidForSaving(OutPackageName, OutError);
	}

	return false;
}

void FEditorSessionSourceFilterService::OnSaveAsPreset()
{
	FString PackageName;
	if (!GetSavePresetPackageName(PackageName))
	{
		return;
	}

	FScopedTransaction Transaction(*FEditorSessionSourceFilterService::TransactionContext, LOCTEXT("SaveAsPreset", "Save As Preset"), nullptr);

	// Saving into a new package
	const FString NewAssetName = FPackageName::GetLongPackageAssetName(PackageName);
	UPackage* NewPackage = CreatePackage(*PackageName);
	USourceFilterCollection* NewPreset = NewObject<USourceFilterCollection>(NewPackage, *NewAssetName, RF_Public | RF_Standalone | RF_Transactional);

	if (NewPreset)
	{
		NewPreset->CopyData(FilterCollection);
		NewPreset->MarkPackageDirty();
		FAssetRegistryModule::AssetCreated(NewPreset);

		// Forcefully save the preset when initially created 
		FString PackageFilename;
		if (FPackageName::TryConvertLongPackageNameToFilename(NewPackage->GetName(), PackageFilename, FPackageName::GetAssetPackageExtension()))
		{
			SavePackageHelper(NewPackage, PackageFilename);
		}
	}
}

TSharedPtr<FExtender> FEditorSessionSourceFilterService::GetExtender()
{
	if (!Extender.IsValid())
	{
		Extender = MakeShared<FExtender>();

		const FName FilterSectionName = TEXT("FilterOptionsMenu");
		Extender->AddMenuExtension(FilterSectionName, EExtensionHook::After, TSharedPtr<FUICommandList>(), 
			FMenuExtensionDelegate::CreateLambda([this](FMenuBuilder& MenuBuilder) -> void
			{
				MenuBuilder.AddSubMenu(LOCTEXT("FilterPresetsLabel", "Filter Presets"),
					LOCTEXT("FilterPresetsTooltip", "Saves currently set of Filters as the default setup."),
					FNewMenuDelegate::CreateLambda([this](FMenuBuilder& SubMenuBuilder)
						{
							SubMenuBuilder.BeginSection(NAME_None, LOCTEXT("FiltersPresetLabel", "Filter Preset(s)"));

							SubMenuBuilder.AddMenuEntry(
								LOCTEXT("SaveFiltersPresetLabel", "Save As"),
								LOCTEXT("SaveFiltersPresetTooltip", "Saves the currently set of Filters as a preset."),
								FSlateIcon(FSourceFilterStyle::Get().GetStyleSetName(), "SourceFilter.SavePreset"),
								FUIAction(
									FExecuteAction::CreateLambda([this]()
									{
										OnSaveAsPreset();
									}),
									FCanExecuteAction::CreateLambda([this]()
									{
										return FilterCollection->GetFilters().Num() > 0;
									})
								)
							);

							SubMenuBuilder.AddMenuSeparator();

							FAssetPickerConfig AssetPickerConfig;
							{
								AssetPickerConfig.SelectionMode = ESelectionMode::Single;
								AssetPickerConfig.InitialAssetViewType = EAssetViewType::Column;
								AssetPickerConfig.bFocusSearchBoxWhenOpened = true;
								AssetPickerConfig.bAllowNullSelection = false;
								AssetPickerConfig.bShowBottomToolbar = true;
								AssetPickerConfig.bAutohideSearchBar = false;
								AssetPickerConfig.bAllowDragging = false;
								AssetPickerConfig.bCanShowClasses = false;
								AssetPickerConfig.bShowPathInColumnView = true;
								AssetPickerConfig.bShowTypeInColumnView = false;
								AssetPickerConfig.bSortByPathInColumnView = false;

								AssetPickerConfig.AssetShowWarningText = LOCTEXT("NoPresets_Warning", "No Presets Found");
								AssetPickerConfig.Filter.ClassPaths.Add(USourceFilterCollection::StaticClass()->GetClassPathName());
								AssetPickerConfig.OnAssetDoubleClicked = FOnAssetDoubleClicked::CreateLambda([this](const FAssetData& InAssetData)
								{
									if (UObject* Asset = InAssetData.GetAsset())
									{
										if (USourceFilterCollection* PresetCollection = Cast<USourceFilterCollection>(Asset))
										{
											FilterCollection->CopyData(PresetCollection);
										}
									}
								});
							}

							IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();
							SubMenuBuilder.AddWidget(SNew(SBox)
								.MinDesiredWidth(250.f)
								.MinDesiredHeight(400.f)
								[
									ContentBrowser.CreateAssetPicker(AssetPickerConfig)
								], FText::GetEmpty()
							);

							SubMenuBuilder.EndSection();
						}
					)
				);
			})
		);

		Extender->AddMenuExtension(FName("FilterPicker"), EExtensionHook::After, TSharedPtr<FUICommandList>(), FMenuExtensionDelegate::CreateLambda([this](FMenuBuilder& MenuBuilder) -> void
		{
			MenuBuilder.AddMenuEntry(LOCTEXT("NewSourceFilterBPLabel", "Create new Filter Blueprint"),
					LOCTEXT("NewSourceFilterBPLabel", "Create new Filter Blueprint"),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateLambda([this]()
					{
						FString AssetPath;
						const FString DefaultFilesystemDirectory = FEditorDirectories::Get().GetLastDirectory(ELastDirectory::NEW_ASSET);
						if (DefaultFilesystemDirectory.IsEmpty() || !FPackageName::TryConvertFilenameToLongPackageName(DefaultFilesystemDirectory, AssetPath))
						{
							// No saved path, just use the game content root
							AssetPath = TEXT("/Game");
						}
						
						// Let user determine new path for Blueprint asset
						FSaveAssetDialogConfig SaveAssetDialogConfig;
						SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("SaveAssetDialogTitle", "Save Asset As");
						SaveAssetDialogConfig.DefaultPath = AssetPath;
						SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::AllowButWarn;

						FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
						FString SaveObjectPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);

						// Check that we have a valid object path to create the blueprint
						if (!SaveObjectPath.IsEmpty())
						{
							const FString SavePackageName = FPackageName::ObjectPathToPackageName(SaveObjectPath);
							const FString SavePackagePath = FPaths::GetPath(SavePackageName);
							const FString SaveAssetName = FPaths::GetBaseFilename(SavePackageName);
							FEditorDirectories::Get().SetLastDirectory(ELastDirectory::NEW_ASSET, SavePackagePath);

							IAssetTools& AssetTools = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools").Get();
							UBlueprintFactory* Factory = NewObject<UBlueprintFactory>();
							Factory->ParentClass = UDataSourceFilter::StaticClass();

							// Generate blueprint with DataSourceFilter as its parent class
							if (UBlueprint* NewFilterBlueprint = Cast<UBlueprint>(AssetTools.CreateAsset(SaveAssetName, SavePackagePath, UBlueprint::StaticClass(), Factory)))
							{
								if (GEditor)
								{
									GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(NewFilterBlueprint);
									AddFilter(NewFilterBlueprint->GeneratedClass.Get()->GetPathName());
								}
							}
						}
					}))
				);
		}));
	}

	return Extender;
}

void FEditorSessionSourceFilterService::GetWorldObjects(TArray<TSharedPtr<FWorldObject>>& OutWorldObjects)
{
	HashToWorld.Empty();

	const TArray<const UWorld*> Worlds = FTraceWorldFiltering::GetWorlds();
	for (const UWorld* World : Worlds)
	{
		FString WorldName;
		FTraceWorldFiltering::GetWorldDisplayString(World, WorldName);
		const uint32 WorldHash = GetTypeHash(World);

		TSharedPtr<FWorldObject> WorldObject = MakeShared<FWorldObject>(WorldName, (uint8)World->WorldType, CAN_TRACE_OBJECT(World), WorldHash);
		OutWorldObjects.Add(WorldObject);
		HashToWorld.Add(WorldHash, World);
	}
}

void FEditorSessionSourceFilterService::SetWorldTraceability(TSharedRef<FWorldObject> InWorldObject, bool bState)
{
	if (const UWorld** WorldPtr = HashToWorld.Find(InWorldObject->GetHash()))
	{
		FTraceWorldFiltering::SetWorldState(*WorldPtr, bState);
	}
}

const TArray<TSharedPtr<IWorldTraceFilter>>& FEditorSessionSourceFilterService::GetWorldFilters()
{
	return WorldFilters;
}

void FEditorSessionSourceFilterService::AddClassFilter(const FString& ActorClassName)
{
	FSoftClassPath ClassPath(ActorClassName);
	if (UClass* Class = ClassPath.TryLoadClass<AActor>())
	{
		const FScopedTransaction Transaction(*FEditorSessionSourceFilterService::TransactionContext, LOCTEXT("AddClassFilter", "Adding Class Filter"), FilterCollection);

		FilterCollection->AddClassFilter(Class);
	}
}

void FEditorSessionSourceFilterService::RemoveClassFilter(TSharedRef<FClassFilterObject> ClassFilterObject)
{
	const FScopedTransaction Transaction(*FEditorSessionSourceFilterService::TransactionContext, LOCTEXT("RemoveClassFilter", "Removing Class Filter"), FilterCollection);

	FilterCollection->RemoveClassFilter(ClassFilterObject->GetClass());
}

void FEditorSessionSourceFilterService::GetClassFilters(TArray<TSharedPtr<FClassFilterObject>>& OutClasses) const
{
	for (const FActorClassFilter& FilterClass : FilterCollection->GetClassFilters())
	{
		OutClasses.Add(MakeShared<FClassFilterObject>(FilterClass.ActorClass.TryLoadClass<AActor>(), FilterClass.bIncludeDerivedClasses));
	}
}

void FEditorSessionSourceFilterService::SetIncludeDerivedClasses(TSharedRef<FClassFilterObject> ClassFilterObject, bool bIncluded)
{
	const FScopedTransaction Transaction(*FEditorSessionSourceFilterService::TransactionContext, LOCTEXT("AddClassFilter", "Adding Class Filter"), FilterCollection);

	FilterCollection->UpdateClassFilter(ClassFilterObject->GetClass(), bIncluded);
}

void FEditorSessionSourceFilterService::PostUndo(bool bSuccess)
{
	StateChanged();
}

void FEditorSessionSourceFilterService::PostRedo(bool bSuccess)
{
	StateChanged();
}

bool FEditorSessionSourceFilterService::MatchesContext(const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjectContexts) const
{
	TArray<UClass*> MatchingClasses = { UDataSourceFilter::StaticClass(), UDataSourceFilterSet::StaticClass(), USourceFilterCollection::StaticClass() };

	return TransactionObjectContexts.ContainsByPredicate([&MatchingClasses](TPair<UObject*, FTransactionObjectEvent>& Pair)
	{
		return MatchingClasses.ContainsByPredicate([Pair](UClass* InClass)
		{
			return Pair.Key->GetClass()->IsChildOf(InClass);
		});
	});
}

void FEditorSessionSourceFilterService::MakeFilterSet(TSharedRef<const IFilterObject> ExistingFilter, TSharedRef<const IFilterObject> ExistingFilterOther)
{
	const FScopedTransaction Transaction(*FEditorSessionSourceFilterService::TransactionContext, LOCTEXT("CreateFilterSet", "Creating Filter Set"), FilterCollection);
	FilterCollection->Modify();

	FilterCollection->MakeFilterSet(CastChecked<UDataSourceFilter>(ExistingFilter->GetFilter()), CastChecked<UDataSourceFilter>(ExistingFilterOther->GetFilter()), EFilterSetMode::AND);
}

void FEditorSessionSourceFilterService::MakeFilterSet(TSharedRef<const IFilterObject> ExistingFilter, EFilterSetMode Mode)
{
	const FScopedTransaction Transaction(*FEditorSessionSourceFilterService::TransactionContext, LOCTEXT("CreateFilterSet", "Creating Filter Set"), FilterCollection);
	FilterCollection->Modify();

	FilterCollection->ConvertFilterToSet(CastChecked<UDataSourceFilter>(ExistingFilter->GetFilter()), Mode);
}

void FEditorSessionSourceFilterService::MakeTopLevelFilter(TSharedRef<const IFilterObject> Filter)
{
	const FScopedTransaction Transaction(*FEditorSessionSourceFilterService::TransactionContext, LOCTEXT("MakeTopLevelFilter", "Make Top Level Filter"), FilterCollection);
	FilterCollection->Modify();

	FilterCollection->MoveFilter(CastChecked<UDataSourceFilter>(Filter->GetFilter()), nullptr);
}

void FEditorSessionSourceFilterService::PopulateTreeView(FTreeViewDataBuilder& InBuilder)
{
	const TArray<UDataSourceFilter*>& Filters = FilterCollection->GetFilters();
	for (UDataSourceFilter* Filter : Filters)
	{
		if (Filter)
		{
			TSharedRef<IFilterObject> FilterObject = AddFilterObjectToDataBuilder(Filter, InBuilder);
			InBuilder.AddFilterObject(FilterObject);
		}
	}
}

void FEditorSessionSourceFilterService::AddReferencedObjects(FReferenceCollector& Collector)
{
	for (auto& Blueprint : DelegateRegisteredBlueprints)
	{
		Collector.AddReferencedObject(Blueprint);
	}
}

TSharedRef<IFilterObject> FEditorSessionSourceFilterService::AddFilterObjectToDataBuilder(UDataSourceFilter* Filter, FTreeViewDataBuilder& InBuilder)
{
	if (UDataSourceFilterSet* FilterSet = Cast<UDataSourceFilterSet>(Filter))
	{
		// Deal with children as well
		TArray<TSharedPtr<IFilterObject>> ChildObjects;
		for (UDataSourceFilter* ChildFilter : FilterSet->GetFilters())
		{
			if (ChildFilter)
			{
				TSharedRef<IFilterObject> ChildFilterObject = AddFilterObjectToDataBuilder(ChildFilter, InBuilder);
				ChildObjects.Add(ChildFilterObject);
			}
		}

		TSharedRef<IFilterObject> FilterSetObject = MakeShared<FFilterSetObject>(*Cast<IDataSourceFilterSetInterface>(FilterSet), *Cast<IDataSourceFilterInterface>(Filter), ChildObjects, AsShared());

		InBuilder.AddChildFilterObject(ChildObjects, FilterSetObject);

		return FilterSetObject;
	}
	else
	{
		TSharedRef<IFilterObject> FilterObject = MakeShared<FFilterObject>(*Cast<IDataSourceFilterInterface>(Filter), AsShared());
		
		if (Filter->GetClass()->ClassGeneratedBy)
		{
			UBlueprint* Blueprint = CastChecked<UBlueprint>(Filter->GetClass()->ClassGeneratedBy);
			Blueprint->OnCompiled().AddSP(this, &FEditorSessionSourceFilterService::OnBlueprintCompiled);
			DelegateRegisteredBlueprints.Add(Blueprint);
		}

		return FilterObject;
	}
}

void FEditorSessionSourceFilterService::SetupWorldFilters()
{
	WorldFilters.Add(
		MakeShared<FWorldTypeTraceFilter>([this](uint8 TypeValue, bool bState) -> void
		{
			FTraceWorldFiltering::SetStateByWorldType((EWorldType::Type)TypeValue, bState);
		},
		[this](uint8 TypeValue) -> bool
		{
			return FTraceWorldFiltering::IsWorldTypeTraceable((EWorldType::Type)TypeValue);
		})
	);

	WorldFilters.Add(
		MakeShared<FWorldNetModeTraceFilter>([this](uint8 TypeValue, bool bState) -> void
		{
			FTraceWorldFiltering::SetStateByWorldNetMode((ENetMode)TypeValue, bState);
		},
		[this](uint8 TypeValue) -> bool
		{
			return FTraceWorldFiltering::IsWorldNetModeTraceable((ENetMode)TypeValue);
		})
	);
}

void FEditorSessionSourceFilterService::OnAssetsPendingDelete(TArray<UObject*> const& ObjectsForDelete)
{
	DelegateRegisteredBlueprints.RemoveAll([&ObjectsForDelete](UBlueprint* Blueprint)
	{
		return ObjectsForDelete.Contains(Blueprint);
	});
}

void FEditorSessionSourceFilterService::OnBlueprintCompiled(UBlueprint* InBlueprint)
{
	if (InBlueprint)
	{
		for (UBlueprint* Blueprint : DelegateRegisteredBlueprints)
		{
			Blueprint->OnCompiled().RemoveAll(this);
		}
		DelegateRegisteredBlueprints.Empty();
	}
}

void FEditorSessionSourceFilterService::OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementsMap)
{
	if (FilterCollection)
	{
		FilterCollection->OnObjectsReplaced(ReplacementsMap);
	}
}

#endif // WITH_ENGINE

#undef LOCTEXT_NAMESPACE // "FEditorSourceFilterService"
