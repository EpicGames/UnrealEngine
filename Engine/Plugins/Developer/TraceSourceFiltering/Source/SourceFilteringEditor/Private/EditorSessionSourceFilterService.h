// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "ISessionSourceFilterService.h"

#include "UObject/GCObject.h"
#include "EditorUndoClient.h"


class UBlueprint;
class USourceFilterCollection;

class UDataSourceFilter;
class UDataSourceFilterSet;

/** Editor implementation of ISessionSourceFilterService, interfaces directly with Engine level filtering systems and settings */
class FEditorSessionSourceFilterService : public ISessionSourceFilterService, public FGCObject, public FSelfRegisteringEditorUndoClient
{
public:
	FEditorSessionSourceFilterService();
	virtual ~FEditorSessionSourceFilterService();

	/** Begin ISessionSourceFilterService overrides */
	virtual void PopulateTreeView(FTreeViewDataBuilder& InBuilder) override;	
	virtual void AddFilter(const FString& FilterClassName) override;
	virtual void AddFilterToSet(TSharedRef<const IFilterObject> FilterSet, const FString& FilterClassName) override;
	virtual void AddFilterToSet(TSharedRef<const IFilterObject> FilterSet, TSharedRef<const IFilterObject> ExistingFilter) override;
	virtual void MakeFilterSet(TSharedRef<const IFilterObject> ExistingFilter, EFilterSetMode Mode) override;
	virtual void MakeFilterSet(TSharedRef<const IFilterObject> ExistingFilter, TSharedRef<const IFilterObject> ExistingFilterOther) override;
	virtual void MakeTopLevelFilter(TSharedRef<const IFilterObject> Filter) override;
	virtual void RemoveFilter(TSharedRef<const IFilterObject> InFilter)  override;
	virtual void SetFilterSetMode(TSharedRef<const IFilterObject> InFilter, EFilterSetMode Mode) override;
	virtual void SetFilterState(TSharedRef<const IFilterObject> InFilter, bool bState) override;
	virtual void ResetFilters() override;	
	virtual FOnSessionStateChanged& GetOnSessionStateChanged() override { return OnSessionStateChanged; }
	virtual void UpdateFilterSettings(UTraceSourceFilteringSettings* InSettings) override;
	virtual UTraceSourceFilteringSettings* GetFilterSettings() override;
	virtual bool IsActionPending() const override;
	virtual TSharedRef<SWidget> GetFilterPickerWidget(FOnFilterClassPicked InFilterClassPicked) override;
	virtual TSharedRef<SWidget> GetClassFilterPickerWidget(FOnFilterClassPicked InFilterClassPicked) override;
	virtual TSharedPtr<FExtender> GetExtender() override;
	virtual void GetWorldObjects(TArray<TSharedPtr<FWorldObject>>& OutWorldObjects) override;
	virtual void SetWorldTraceability(TSharedRef<FWorldObject> InWorldObject, bool bState) override;
	virtual const TArray<TSharedPtr<IWorldTraceFilter>>& GetWorldFilters() override;
	virtual void AddClassFilter(const FString& ActorClassName) override;
	virtual void RemoveClassFilter(TSharedRef<FClassFilterObject> ClassFilterObject) override;
	virtual void GetClassFilters(TArray<TSharedPtr<FClassFilterObject>>& OutClasses) const override;
	virtual void SetIncludeDerivedClasses(TSharedRef<FClassFilterObject> ClassFilterObject, bool bIncluded) override;
	/** End ISessionSourceFilterService overrides */	

	/** Begin FEditorUndoClient overrides*/
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	virtual bool MatchesContext(const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjectContexts) const;
	/** End FEditorUndoClient overrides */

	/** Begin FGCObject overrides*/
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FEditorSessionSourceFilterService");
	}
	/** End FGCObject overrides */

protected:
	/** Fires the OnSessionStateChanged delegate, indicating that the contained data has changed  */
	void StateChanged();

	/** Callback for whenever the user opts to save the current filtering state as a preset */
	void OnSaveAsPreset();

	/** Callback for whenever any Filter instance blueprint class gets (re)compiled */
	void OnBlueprintCompiled(UBlueprint* InBlueprint);

	/** Callback for whenever any Filter blueprint instances are reinstanced */
	void OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementsMap);

	/** Helper function to populate FTreeViewDataBuilder with a specific FilterObject */
	TSharedRef<IFilterObject> AddFilterObjectToDataBuilder(UDataSourceFilter* Filter, FTreeViewDataBuilder& InBuilder);

	void SetupWorldFilters();
	
	/** Callback for whenever any Filter instance blueprint class gets deleted */
	void OnAssetsPendingDelete(TArray<UObject*> const& ObjectsForDelete);
protected:
	/** Blueprints on which a OnCompiled calllback has been registered, used to refresh data and repopulate the UI */
	TArray<TObjectPtr<UBlueprint>> DelegateRegisteredBlueprints;

	/** Filter collection this session represents, retrieved from FTraceSourceFiltering */
	USourceFilterCollection* FilterCollection;

	/** Delegate used to rely data updates to notify anyone interested */
	FOnSessionStateChanged OnSessionStateChanged;
	
	/** UI extender, used to insert preset functionality into STraceSourceFilteringWindow */
	TSharedPtr<FExtender> Extender;

	/** World filtering data */
	TMap<uint32, const UWorld*> HashToWorld;
	TArray<TSharedPtr<IWorldTraceFilter>> WorldFilters;

	/** Transaction context naming for all scoped transactions performed in thsi class */
	static FString TransactionContext;

};

#endif // WITH_ENGINE
