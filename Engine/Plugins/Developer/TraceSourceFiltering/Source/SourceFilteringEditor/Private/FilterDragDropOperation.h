// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"

#include "SourceFilterStyle.h"

class IFilterObject;

class FFilterDragDropOp : public FDragDropOperation
{
public:
	DRAG_DROP_OPERATOR_TYPE(FFilterDragDropOp, FDragDropOperation)

	TWeakPtr<class IFilterObject> FilterObject;

	/** The widget decorator to use */
	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override
	{
		return SNew(SBorder)
			.BorderImage(FSourceFilterStyle::GetBrush("SourceFilter.DragDrop.Border"))
			.Content()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2, 1, 0, 0)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.TextStyle(FSourceFilterStyle::Get(), "SourceFilter.TextStyle")
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.12"))
					.Text(this, &FFilterDragDropOp::GetIconText)
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2, 0, 0, 0)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(this, &FFilterDragDropOp::GetHoverText)
				]
			];
	}

	FText GetHoverText() const
	{
		return Text;	
	}

	FText GetIconText() const
	{
		return IconText;
	}

	void SetIconText(FText InText)
	{
		IconText = InText;
	}

	void SetText(FText InText)
	{
		Text = InText;
	}

	void Reset()
	{
		Text = FText::FromString(TEXT("Invalid Operation"));
		IconText = FText::FromString(FString(TEXT("\xf05e")));
	}
	
	static TSharedRef<FFilterDragDropOp> New(TSharedRef<class IFilterObject> InFilterObject)
	{
		TSharedRef<FFilterDragDropOp> Operation = MakeShareable(new FFilterDragDropOp);
		Operation->FilterObject = InFilterObject;
		Operation->Reset();
		Operation->Construct();
		return Operation;
	}

private:
	FText IconText;
	FText Text;
};
