// Copyright Epic Games, Inc. All Rights Reserved.

#include "FilterObject.h"

#include "Widgets/Layout/SWrapBox.h"

#include "ISessionSourceFilterService.h"
#include "FilterDragDropOperation.h"
#include "IDataSourceFilterInterface.h"

#if WITH_EDITOR
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "FilterObject"

FFilterObject::FFilterObject(IDataSourceFilterInterface& InFilter, TSharedRef<ISessionSourceFilterService> InSessionFilterService) : SessionFilterService(InSessionFilterService)
{
	WeakInterface = &InFilter;
}

FText FFilterObject::GetDisplayText() const
{
	FText Text;
	if (GetFilter())
	{
		WeakInterface->Execute_GetDisplayText(WeakInterface.GetObject(), Text);

		const FDataSourceFilterConfiguration& Configuration = WeakInterface->GetConfiguration();
		FText PrefixText;
		FText PostFixText;
		
		if (Configuration.bOnlyApplyDuringActorSpawn)
		{
			PrefixText = LOCTEXT("OnSpawnLabel", "OnSpawn: ");
		}
		else if (Configuration.bCanRunAsynchronously)
		{
			PrefixText = LOCTEXT("AsyncLabel", "Asynchronously: ");
		}

		if (Configuration.FilterApplyingTickInterval != 1)
		{
			PostFixText = FText::Format(LOCTEXT("PostFixConfigurationTextFormat", " [Applied every {0} frames]"), { FText::FromString(FString::FromInt(Configuration.FilterApplyingTickInterval)) });
		}

		Text = FText::Format(LOCTEXT("DisplayTextFormat", "{0}{1}{2}"), { PrefixText, Text, PostFixText } );
	}
	return Text;
}

FText FFilterObject::GetToolTipText() const
{
	FText Text;
	if (GetFilter())
	{
		WeakInterface->Execute_GetToolTipText(WeakInterface.GetObject(), Text);
	}
	return Text;
}

UObject* FFilterObject::GetFilter() const
{
	return WeakInterface.GetObject();
}

bool FFilterObject::IsFilterEnabled() const
{
	if (GetFilter())
	{
		return WeakInterface->IsEnabled();
	}

	return false;
}

TSharedRef<SWidget> FFilterObject::MakeWidget(TSharedRef<SWrapBox> ParentWrapBox) const
{
	TSharedPtr<SHorizontalBox> WidgetBox;

	ParentWrapBox->AddSlot()
	[
		SAssignNew(WidgetBox, SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0, 0, 2, 0)
		[
			SNew(SBorder)
			.BorderImage(FSourceFilterStyle::GetBrush("SourceFilter.FilterBrush"))
			.BorderBackgroundColor(FLinearColor(0.25f, 0.25f, 0.25f, 0.9f))
			.ForegroundColor(FLinearColor::White)
			.Padding(FMargin(4.f))
			[
				SNew(STextBlock)
				.TextStyle(FSourceFilterStyle::Get(), "SourceFilter.TextStyle")
				.Text_Lambda([this]() { return GetDisplayText(); })
				.ToolTipText_Lambda([this]() { return GetToolTipText(); })
			]
		]
	];

	TAttribute<bool> FilterEnabled = TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FFilterObject::IsFilterEnabled));
	WidgetBox->SetEnabled(FilterEnabled);

	return ParentWrapBox;
}

FReply FFilterObject::HandleDragEnter(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FFilterDragDropOp> FilterDragOp = DragDropEvent.GetOperationAs<FFilterDragDropOp>();

	if (FilterDragOp.IsValid())
	{
		// Check if the dragged object is actually a Filter 
		if (FilterDragOp->FilterObject.Pin()->GetFilter()->Implements<UDataSourceFilterInterface>())
		{
			// Ensure we are not dragging onto ourselves, if not allow for Creating a Filter Set containing both the dragged and this Filter
			if (FilterDragOp->FilterObject.Pin() != AsShared())
			{
				FilterDragOp->SetIconText(FText::FromString(TEXT("\xf0fe")));
				FilterDragOp->SetText(LOCTEXT("CreateFilterToSetDragOpLabel", "Create Filter Set"));
			}
			else
			{
				FilterDragOp->Reset();
			}

			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

void FFilterObject::HandleDragLeave(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FFilterDragDropOp> FilterDragOp = DragDropEvent.GetOperationAs<FFilterDragDropOp>();
	if (FilterDragOp.IsValid())
	{
		FilterDragOp->Reset();
	}
}

FReply FFilterObject::HandleDrop(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FFilterDragDropOp> FilterDragOp = DragDropEvent.GetOperationAs<FFilterDragDropOp>();

	if (FilterDragOp.IsValid())
	{
		if (FilterDragOp->FilterObject.Pin()->GetFilter()->Implements<UDataSourceFilterInterface>() && FilterDragOp->FilterObject.Pin() != AsShared())
		{
			// Request to create a filter set contain both the dragged filter and this
			SessionFilterService->MakeFilterSet(AsShared(), FilterDragOp->FilterObject.Pin()->AsShared());
			return FReply::Handled();
		}

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE // "FilterObject"
