// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IFilterObject.h"
#include "UObject/WeakInterfacePtr.h"

class ISessionSourceFilterService;
class IDataSourceFilterInterface;

class FFilterObject : public IFilterObject
{
public:
	FFilterObject(IDataSourceFilterInterface& InFilter, TSharedRef<ISessionSourceFilterService> InSessionFilterService);

	/** Begin IFilterObject overrides */
	virtual FText GetDisplayText() const;
	virtual FText GetToolTipText() const;
	virtual UObject* GetFilter() const;
	virtual bool IsFilterEnabled() const;	
	virtual TSharedRef<SWidget> MakeWidget(TSharedRef<SWrapBox> ParentWrapBox) const;
	virtual FReply HandleDragEnter(const FDragDropEvent& DragDropEvent) override;
	virtual void HandleDragLeave(const FDragDropEvent& DragDropEvent) override;
	virtual FReply HandleDrop(const FDragDropEvent& DragDropEvent) override;
	/** End IFilterObject overrides */

protected:
	/** Weak interface pointer retrieved from Data Source Filter instance this object represents */
	TWeakInterfacePtr<IDataSourceFilterInterface> WeakInterface;
	TSharedPtr<ISessionSourceFilterService> SessionFilterService;
};
