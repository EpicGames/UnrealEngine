// Copyright Epic Games, Inc. All Rights Reserved.

#include "FilterSetObject.h"

#include "DataSourceFiltering.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Framework/Commands/UICommandList.h"
#include "Widgets/Input/SComboButton.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ISessionSourceFilterService.h"
#include "IDataSourceFilterInterface.h"
#include "IDataSourceFilterSetInterface.h"
#include "FilterDragDropOperation.h"

#define LOCTEXT_NAMESPACE "FilterSetObject"

FFilterSetObject::FFilterSetObject(IDataSourceFilterSetInterface& InFilterSet, IDataSourceFilterInterface& InFilter, const TArray<TSharedPtr<IFilterObject>>& InChildFilters, TSharedRef<ISessionSourceFilterService> InSessionFilterService) : ChildFilters(InChildFilters), SessionFilterService(InSessionFilterService), FilterSetModeEnumPtr(FindObject<UEnum>(nullptr, TEXT("/Script/SourceFilteringCore.EFilterSetMode"), EFindObjectFlags::ExactClass))
{
	WeakFilter = &InFilter;
	WeakFilterSet = &InFilterSet;
}

FText FFilterSetObject::GetDisplayText() const
{
	FText Text;
	if (GetFilter())
	{
		WeakFilter->GetDisplayText_Implementation(Text);
	}
	return Text;
}

FText FFilterSetObject::GetToolTipText() const
{
	FText Text;
	if (GetFilter())
	{
		WeakFilter->GetToolTipText_Implementation(Text);
	}
	return Text;
}

UObject* FFilterSetObject::GetFilter() const
{
	return WeakFilter.GetObject();
}

bool FFilterSetObject::IsFilterEnabled() const
{
	if (GetFilter())
	{
		return WeakFilter->IsEnabled();
	}

	return false;
}

void FFilterSetObject::AddFilterSetModeWidget(TSharedRef<SWrapBox> ParentWrapBox) const
{
	ParentWrapBox->AddSlot()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0, 0, 2, 0)
		.VAlign(VAlign_Center)
		[
			SNew(SComboButton)
			.ComboButtonStyle(FSourceFilterStyle::Get(), "SourceFilter.ComboButton")
			.ButtonStyle(FSourceFilterStyle::Get(), FName(*("SourceFilter.FilterSetOperation." + FilterSetModeEnumPtr->GetNameStringByValue((int64)WeakFilterSet->GetFilterSetMode()))))
			.ForegroundColor(FLinearColor::White)
			.ContentPadding(2.0f)
			.HasDownArrow(false)
			.VAlign(VAlign_Center)
			.ToolTipText_Lambda([this]() -> FText
			{
				return FText::Format(LOCTEXT("FilterSetModeTooltip", "{0} operator is used for this Filter Set."), FilterSetModeEnumPtr->GetDisplayNameTextByValue((int64)WeakFilterSet->GetFilterSetMode()));
			})
			.OnGetMenuContent(FOnGetContent::CreateLambda([this]()
			{
				FMenuBuilder MenuBuilder(true, TSharedPtr<FUICommandList>());

				const FText LabelTextFormat = LOCTEXT("ChangeFilterSetModeLabel", "{0}");
				const FText ToolTipTextFormat = LOCTEXT("ChangeFilterSetModeTooltip", "Set Filtering Mode to {0} operator.");

				MenuBuilder.BeginSection(NAME_None, LOCTEXT("FilterModeSectionLabel", "Filter Modes"));

				// Add an entry for each Filter Set Mode, except for the one currently set for this Filter Set
				for (EFilterSetMode Mode : TEnumRange<EFilterSetMode>())
				{
					if (Mode != WeakFilterSet->GetFilterSetMode())
					{
						const FText ModeText = FilterSetModeEnumPtr->GetDisplayNameTextByValue((int64)Mode);

						MenuBuilder.AddMenuEntry(
							FText::Format(LabelTextFormat, ModeText),
							FText::Format(ToolTipTextFormat, ModeText),
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateLambda([this, Mode]() 
								{
									// Request a change of this Filter Set's mode through the session service
									SessionFilterService->SetFilterSetMode(AsShared(), Mode); 
								})
							)
						);
					}
				}

				MenuBuilder.EndSection();

				return MenuBuilder.MakeWidget();
			}))
			.ButtonContent()
			[
				SNew(STextBlock)
				.TextStyle(FSourceFilterStyle::Get(), "SourceFilter.TextStyle")
				.Text_Lambda([this]()
				{
					return FilterSetModeEnumPtr->GetDisplayNameTextByValue((int64)WeakFilterSet->GetFilterSetMode());
				})
			]
		]
	];
}

TSharedRef<SWidget> FFilterSetObject::MakeWidget(TSharedRef<SWrapBox> ParentWrapBox) const
{
	auto AddEncapsulatingWidget = [ParentWrapBox](FText InText)
	{
		ParentWrapBox->AddSlot()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0, 0, 2, 0)
			[
				SNew(SBorder)
				.BorderImage(FSourceFilterStyle::GetBrush("SourceFilter.FilterSetBrush"))
				.ForegroundColor(FLinearColor::White)
				.Padding(FMargin(4.f))
				[
					SNew(STextBlock)
					.TextStyle(FSourceFilterStyle::Get(), "SourceFilter.TextStyle")
					.Text(InText)
				]
			]
		];
	};

	AddEncapsulatingWidget(LOCTEXT("OpenBracket", "("));	

	// In case this is a NOT filter, only add the operation widget at the start of the filter set as such: (!FilterA, FilterB)
	const bool bIsNOTFilterSet = WeakFilterSet->GetFilterSetMode() == EFilterSetMode::NOT;
	if (bIsNOTFilterSet)
	{
		AddFilterSetModeWidget(ParentWrapBox);
	}

	for (const TSharedPtr<IFilterObject>& ChildFilter : ChildFilters)
	{
		ChildFilter->MakeWidget(ParentWrapBox);

		// In case this is not a NOT filter, add the operation widget in between each filter (or at the end of a filter set containing a single filter) as such: (FilterA AND FilterB)
		const bool bIsLastFilter = ChildFilter != ChildFilters.Last();
		const bool bShouldAddFilterSetWidget = bIsLastFilter || (ChildFilters.Num() == 1 && !bIsNOTFilterSet);
		if (bShouldAddFilterSetWidget)
		{
			AddFilterSetModeWidget(ParentWrapBox);
		}
	}

	AddEncapsulatingWidget(LOCTEXT("CloseBracket", ")"));

	return ParentWrapBox;
}

FReply FFilterSetObject::HandleDragEnter(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FFilterDragDropOp> FilterDragOp = DragDropEvent.GetOperationAs<FFilterDragDropOp>();

	if (FilterDragOp.IsValid())
	{
		// Ensure that the dragged object is indeed a filter
		if (FilterDragOp->FilterObject.Pin()->GetFilter()->Implements<UDataSourceFilterInterface>())
		{
			// And that we're not dragging onto ourselves
			if (FilterDragOp->FilterObject.Pin() != AsShared())
			{
				// In case this filter set not yet contains the dragged object, allow for adding it				
				if (!ChildFilters.Contains(FilterDragOp->FilterObject))
				{
					FilterDragOp->SetIconText(FText::FromString(TEXT("\xf0fe")));
					FilterDragOp->SetText(LOCTEXT("AddFilterToSetDragOpLabel", "Add Filter to Filter Set"));
				}
				else
				{
					FilterDragOp->Reset();
					FilterDragOp->SetText(LOCTEXT("AlreadyChildFilterToSetDragOpLabel", "Filter is already part of Filter Set"));
				}
			}
			else
			{
				FilterDragOp->Reset();
			}

			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

void FFilterSetObject::HandleDragLeave(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FFilterDragDropOp> FilterDragOp = DragDropEvent.GetOperationAs<FFilterDragDropOp>();
	if (FilterDragOp.IsValid())
	{
		FilterDragOp->Reset();
	}
}

FReply FFilterSetObject::HandleDrop(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FFilterDragDropOp> FilterDragOp = DragDropEvent.GetOperationAs<FFilterDragDropOp>();

	if (FilterDragOp.IsValid())
	{
		// If the dragged object is a Filter, and is not yet contained by this Filter Set, request to add it
		if (FilterDragOp->FilterObject.Pin()->GetFilter()->Implements<UDataSourceFilterInterface>() && FilterDragOp->FilterObject.Pin() != AsShared() && !ChildFilters.Contains(FilterDragOp->FilterObject))
		{
			SessionFilterService->AddFilterToSet(AsShared(), FilterDragOp->FilterObject.Pin()->AsShared());
			return FReply::Handled();
		}

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE // "FilterSetObject"
