// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IFilterObject.h"
#include "UObject/WeakInterfacePtr.h"

class ISessionSourceFilterService;
class IDataSourceFilterSetInterface;
class IDataSourceFilterInterface;

class FFilterSetObject : public IFilterObject
{
public:
	FFilterSetObject(IDataSourceFilterSetInterface& InFilterSet, IDataSourceFilterInterface& InFilter, const TArray<TSharedPtr<IFilterObject>>& InChildFilters, TSharedRef<ISessionSourceFilterService> InSessionFilterService);

	/** Begin IFilterObject overrides */
	virtual FText GetDisplayText() const;
	virtual FText GetToolTipText() const;
	virtual UObject* GetFilter() const;	
	virtual bool IsFilterEnabled() const;
	virtual TSharedRef<SWidget> MakeWidget(TSharedRef<SWrapBox> ParentWrapBox) const;
	virtual FReply HandleDragEnter(const FDragDropEvent& DragDropEvent) override;
	virtual void HandleDragLeave(const FDragDropEvent& DragDropEvent) override;
	virtual FReply HandleDrop(const FDragDropEvent& DragDropEvent) override;
	/** End IFilterObject overrides */

protected:
	/** Adds appropriate Filter Set operation widget to the provided SWrapBox */
	void AddFilterSetModeWidget(TSharedRef<SWrapBox> ParentWrapBox) const;

protected:
	/** Weak filter interface pointer retrieved from Data Source Filter instance this object represents */
	TWeakInterfacePtr<IDataSourceFilterInterface> WeakFilter;
	/** Weak filter set interface pointer retrieved from Data Source Filter instance this object represents */
	TWeakInterfacePtr<IDataSourceFilterSetInterface> WeakFilterSet;

	/** Any filter objects which are part of this set*/
	TArray<TSharedPtr<IFilterObject>> ChildFilters;

	/** Session filter service which is responsible for generating this object */
	TSharedPtr<ISessionSourceFilterService> SessionFilterService;

	/** Cached pointer to UEnum representing EFilterSetMode */
	const UEnum* FilterSetModeEnumPtr;
};
