// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class FDragDropEvent;
class FReply;

class FText;
class UObject;
class SWidget;
class SWrapBox;

/** Interface for any FilterObject representation throughout the Filtering UI */
class IFilterObject : public TSharedFromThis<IFilterObject>
{
public:
	virtual ~IFilterObject() = default;

	/** Return the display text to be displayed for this object */
	virtual FText GetDisplayText() const = 0;
	
	/** Return the tooltip to be displayed for this object */
	virtual FText GetToolTipText() const = 0;	

	/** Return Filter UObject */
	virtual UObject* GetFilter() const = 0;

	/** Return whether or not this filter object is currently enabled*/
	virtual bool IsFilterEnabled() const = 0;

	/** Generate widget representing this object */
	virtual TSharedRef<SWidget> MakeWidget(TSharedRef<SWrapBox> ParentWrapBox) const = 0;

	/** Handle a drag and drop enter event */
	virtual FReply HandleDragEnter(const FDragDropEvent& DragDropEvent) = 0;

	/** Handle a drag and drop leave event */
	virtual void HandleDragLeave(const FDragDropEvent& DragDropEvent) = 0;

	/** Handle a drag and drop drop event */
	virtual FReply HandleDrop(const FDragDropEvent& DragDropEvent) = 0;
};
