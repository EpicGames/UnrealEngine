// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

class FExtender;

enum class EFilterSetMode : uint8;

class IFilterObject;
struct FTreeViewDataBuilder;
class SWidget;

class FSourceFilterCollection;
class UTraceSourceFilteringSettings;
class FWorldObject;
class FClassFilterObject;

DECLARE_DELEGATE_OneParam(FOnFilterClassPicked, FString /*ClassName*/);

DECLARE_MULTICAST_DELEGATE(FOnSessionStateChanged);

/** Interface for implementing a World filter, setting whether or not a specific UWorld and its contained objects can Trace out data (events) */
class IWorldTraceFilter : public TSharedFromThis<IWorldTraceFilter>
{
public:
	virtual ~IWorldTraceFilter() = default;
	virtual FText GetDisplayText() = 0;
	virtual FText GetToolTipText() = 0;
	virtual TSharedRef<SWidget> GenerateWidget() = 0;
};

/** Interface providing interaction with an running instance's its Trace and World Filtering systems */
class ISessionSourceFilterService : public TSharedFromThis<ISessionSourceFilterService>
{
public:
	virtual ~ISessionSourceFilterService() = default;

	/** Delegate which gets broadcast-ed whenever the contained data (state) changes */
	virtual FOnSessionStateChanged& GetOnSessionStateChanged() = 0;

	/** Returns whether or not a request action is pending complete */
	virtual bool IsActionPending() const = 0;

	/** Adds a Filter instance of the provided UClass name */
	virtual void AddFilter(const FString& FilterClassName) = 0;

	/** Removes a specific Filter (Set) instance */
	virtual void RemoveFilter(TSharedRef<const IFilterObject> InFilter) = 0;
	
	/** Add a Filter instance, of the provided UClass name, to the specified Filter Set instance */
	virtual void AddFilterToSet(TSharedRef<const IFilterObject> FilterSet, const FString& FilterClassName) = 0;	

	/** Add a specific Filter instance to the specified Filter Set instance */
	virtual void AddFilterToSet(TSharedRef<const IFilterObject> FilterSet, TSharedRef<const IFilterObject> ExistingFilter) = 0;

	/** Moves the specific filter to root level in the filtering tree */
	virtual void MakeTopLevelFilter(TSharedRef<const IFilterObject> Filter) = 0;
	
	/** Creates a new Filter Set instance, with the provided filter mode, replacing and containing the specific Filter instance*/
	virtual void MakeFilterSet(TSharedRef<const IFilterObject> ExistingFilter, EFilterSetMode Mode) = 0;

	/** Creates a new Filter Set instance, with default AND filter mode, replacing and containing the specific Filter instances*/
	virtual void MakeFilterSet(TSharedRef<const IFilterObject> ExistingFilter, TSharedRef<const IFilterObject> ExistingFilterOther) = 0;

	/** Sets the state of a specific Filter instance, true= enabled; false= disabled */
	virtual void SetFilterState(TSharedRef<const IFilterObject> InFilter, bool bState) = 0;

	/** Sets the Filtering Mode for a specific Filter Set instance */
	virtual void SetFilterSetMode(TSharedRef<const IFilterObject> InFilter, EFilterSetMode Mode) = 0;

	/** Resets the complete filtering tree, removing all Filter instances */
	virtual void ResetFilters() = 0;

	/** Updates the Filtering Settings for this specific session */
  	virtual void UpdateFilterSettings(UTraceSourceFilteringSettings* InSettings) = 0;
	
	/** Retrieves the current state of the Filtering Settings for this specific session */
	virtual UTraceSourceFilteringSettings* GetFilterSettings() = 0;
	
	/** Request to populate a TreeView using the filter (set) hierarchy */
	virtual void PopulateTreeView(FTreeViewDataBuilder& InBuilder) = 0;
	
	/** Returns a slate widget used for picking a Filter class, which will execute the passed in delegate on selection/completion */
	virtual TSharedRef<SWidget> GetFilterPickerWidget(FOnFilterClassPicked InFilterClassPicked) = 0;

	/** Returns a slate widget used for picking AActor derived UClass, which will execute the passed in delegate on selection/completion */
	virtual TSharedRef<SWidget> GetClassFilterPickerWidget(FOnFilterClassPicked InFilterClassPicked) = 0;	

	/** Returns an FExtender instance which is incorporated anytime a context menu (MenuBuilder) is created */
	virtual TSharedPtr<FExtender> GetExtender() = 0;

	/** Returns an FWorldObject for each active UWorld instance */
	virtual void GetWorldObjects(TArray<TSharedPtr<FWorldObject>>& OutWorldObjects) = 0;
	/** Sets whether or not the UWorld represented by InWorldObject is traceable */
	virtual void SetWorldTraceability(TSharedRef<FWorldObject> InWorldObject, bool bState) = 0;

	/** Returns all currently available World Trace filters */
	virtual const TArray<TSharedPtr<IWorldTraceFilter>>& GetWorldFilters() = 0;

	/** Add a class filter, used to filter AActors on a high-level */
	virtual void AddClassFilter(const FString& ActorClassName) = 0;
	
	/** Removes a specific Class Filter instance */
	virtual void RemoveClassFilter(TSharedRef<FClassFilterObject> ClassFilterObject) = 0;

	/** Returns all Class filters */
	virtual void GetClassFilters(TArray<TSharedPtr<FClassFilterObject>>& OutClasses) const = 0;

	/** Updating whether or not classes derived from the filter class should be included when applying filtering */
	virtual void SetIncludeDerivedClasses(TSharedRef<FClassFilterObject> ClassFilterObject, bool bIncluded) = 0;	
};
