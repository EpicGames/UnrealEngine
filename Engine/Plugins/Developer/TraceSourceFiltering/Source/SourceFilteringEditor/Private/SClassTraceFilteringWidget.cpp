// Copyright Epic Games, Inc. All Rights Reserved.

#include "SClassTraceFilteringWidget.h"

#include "Framework/Commands/UICommandList.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBorder.h"

#include "ISessionSourceFilterService.h"
#include "SourceFilterStyle.h"
#include "ClassFilterObject.h"

#define LOCTEXT_NAMESPACE "SClassTraceFilteringWidget"

void SClassTraceFilteringWidget::Construct(const FArguments& InArgs)
{
	ConstructClassFilterPickerButton();

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(10.0f, 0.f, 2.f, 0.f)
		[
			SAssignNew(ClassFiltersWrapBox, SWrapBox)
			.UseAllottedSize(true)
		]
							
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(10.0f, 0.f, 2.f, 0.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f, 4.f, 0.f, 0.f)
			[
				AddClassFilterButton->AsShared()
			]
		]
	];

	TAttribute<bool> EnabledAttribute = TAttribute<bool>::Create([this]() -> bool
	{
		return SessionFilterService.IsValid() && !SessionFilterService->IsActionPending();
	});

	ClassFiltersWrapBox->SetEnabled(EnabledAttribute);	
}

void SClassTraceFilteringWidget::SetSessionFilterService(TSharedPtr<ISessionSourceFilterService> InSessionFilterService)
{
	if (SessionFilterService.IsValid())
	{
		SessionFilterService->GetOnSessionStateChanged().RemoveAll(this);
	}
	SessionFilterService = InSessionFilterService;
	SessionFilterService->GetOnSessionStateChanged().AddSP(this, &SClassTraceFilteringWidget::RefreshClassFilterData);

	RefreshClassFilterData();
}

void SClassTraceFilteringWidget::ConstructClassFilterPickerButton()
{
	/** Callback for whenever a UClass (name) was selected */	
	auto OnClassFilterPicked = [this](FString PickedClassName)
	{
		if (SessionFilterService.Get())
		{
			SessionFilterService->AddClassFilter(PickedClassName);
			AddClassFilterButton->SetIsOpen(false);
		}
	};

	
	SAssignNew(AddClassFilterButton, SComboButton)
	.Visibility(EVisibility::Visible)
	.ComboButtonStyle(FSourceFilterStyle::Get(), "SourceFilter.ComboButton")	
	.ForegroundColor(FLinearColor::White)
	.ContentPadding(FMargin(2.f, 2.0f))
	.HasDownArrow(false)
	.OnGetMenuContent(FOnGetContent::CreateLambda([OnClassFilterPicked, this]()
	{
		FMenuBuilder MenuBuilder(true, TSharedPtr<FUICommandList>(), SessionFilterService->GetExtender());

		MenuBuilder.BeginSection(FName("ClassFilterPicker"));
		{
			MenuBuilder.AddWidget(SessionFilterService->GetClassFilterPickerWidget(FOnFilterClassPicked::CreateLambda(OnClassFilterPicked)), FText::GetEmpty(), true, false);
		}
		MenuBuilder.EndSection();

		return MenuBuilder.MakeWidget();
	}))
	.ButtonContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.TextStyle(FSourceFilterStyle::Get(), "SourceFilter.TextStyle")
			.Font(FSourceFilterStyle::Get().GetFontStyle("FontAwesome.12"))
			.Text(FText::FromString(FString(TEXT("\xf0fe"))) /*fa-filter*/)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2, 0, 0, 0)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.TextStyle(FSourceFilterStyle::Get(), "SourceFilter.TextStyle")
			.Text(LOCTEXT("FilterMenuLabel", "Add Filter"))
		]
	];
}

void SClassTraceFilteringWidget::RefreshClassFilterData()
{
	ClassFiltersWrapBox->ClearChildren();
	ClassFilterObjects.Reset();
	SessionFilterService->GetClassFilters(ClassFilterObjects);
	
	for (TSharedPtr<FClassFilterObject> Class : ClassFilterObjects)
	{
		ClassFiltersWrapBox->AddSlot()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(0.f, 2.f, 2.f, 2.f))
			.VAlign(VAlign_Center)
			[
				SNew(SComboButton)
				.ComboButtonStyle(FSourceFilterStyle::Get(), "SourceFilter.ComboButton")
				.ButtonStyle(FSourceFilterStyle::Get(), FName(TEXT("SourceFilter.Filter")))
				.ForegroundColor(FLinearColor::White)
				.ContentPadding(2.0f)
				.HasDownArrow(false)
				.VAlign(VAlign_Center)
				.ToolTipText_Lambda([Class]() -> FText
				{
					return Class->GetDisplayText();						
				})
				.OnGetMenuContent(FOnGetContent::CreateLambda([this, Class]()
				{
					FMenuBuilder MenuBuilder(true, TSharedPtr<FUICommandList>());

					MenuBuilder.AddMenuEntry(
						LOCTEXT("IncludeDerivedClassFilterLabel", "Include derived classes"),
						LOCTEXT("IncludeDerivedClassFilterTooltip", "Whether or not derived classes should also be accounted for when filtering."),
						FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateLambda([this, Class]()
							{
								SessionFilterService->SetIncludeDerivedClasses(Class.ToSharedRef(), !Class->IncludesDerivedClasses());
							}),
							FCanExecuteAction::CreateLambda([this]()
							{
								return SessionFilterService.IsValid();
							}),
							FGetActionCheckState::CreateLambda([Class]()
							{
								return Class->IncludesDerivedClasses() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
							})
						),
						NAME_None,
						EUserInterfaceActionType::Check
					);

					MenuBuilder.AddMenuEntry(
						LOCTEXT("RemoveClassFilterLabel", "Remove Filter"),
						LOCTEXT("RemoveClassFilterTooltip", "Removes the Class Filter from the Filtering State."),
						FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateLambda([this, Class]()
							{
								SessionFilterService->RemoveClassFilter(Class.ToSharedRef());
							}),
							FCanExecuteAction::CreateLambda([this]()
							{
								return SessionFilterService.IsValid();
							})
						),
						NAME_None								
					);

					return MenuBuilder.MakeWidget();
				}))
				.ButtonContent()
				[
					SNew(STextBlock)
					.TextStyle(FSourceFilterStyle::Get(), "SourceFilter.TextStyle")
					.Text(Class->GetDisplayText())
				]
			]
		];

		if (Class != ClassFilterObjects.Last())
		{
			ClassFiltersWrapBox->AddSlot()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(0.f, 2.f, 2.f, 2.f))
				.VAlign(VAlign_Center)
				[
					SNew(SBorder)
					.BorderImage(FSourceFilterStyle::GetBrush("SourceFilter.FilterBrush"))
					.BorderBackgroundColor(FLinearColor(0.25f, 0.25f, 0.85f, 0.9f))
					.ForegroundColor(FLinearColor::White)
					.Padding(FMargin(4.f))
					[
						SNew(STextBlock)
						.TextStyle(FSourceFilterStyle::Get(), "SourceFilter.TextStyle")
						.Text(FText::FromString(TEXT("OR")))
					]
				]
			];
		}
	}
}

#undef LOCTEXT_NAMESPACE // "SClassTraceFilteringWidget"
