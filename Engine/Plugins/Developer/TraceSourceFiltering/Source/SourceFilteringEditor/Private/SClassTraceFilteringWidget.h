// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class ISessionSourceFilterService;
class FClassFilterObject;
class SWrapBox;
class SComboButton;

class SClassTraceFilteringWidget : public SCompoundWidget
{
public:
	/** Default constructor. */
	SClassTraceFilteringWidget() {}
	virtual ~SClassTraceFilteringWidget() = default;

	SLATE_BEGIN_ARGS(SClassTraceFilteringWidget) {}
	SLATE_END_ARGS()
	void Construct(const FArguments& InArgs);

	void SetSessionFilterService(TSharedPtr<ISessionSourceFilterService> InSessionFilterService);
protected:
	void ConstructClassFilterPickerButton();
	void RefreshClassFilterData();
protected:
	/** Session service used to retrieve state and request filtering changes */
	TSharedPtr<ISessionSourceFilterService> SessionFilterService;

	/** Latest set of retrieved Class Filter Objects from SessionFilterService */
	TArray<TSharedPtr<FClassFilterObject>> ClassFilterObjects;

	/** Wrap box that will contain the class filter object widgets */
	TSharedPtr<SWrapBox> ClassFiltersWrapBox;

	TSharedPtr<SComboButton> AddClassFilterButton;
};
