// Copyright Epic Games, Inc. All Rights Reserved.

#include "SFilterObjectWidget.h"
#include "IFilterObject.h"
#include "FilterDragDropOperation.h"
#include "Widgets/Layout/SWrapBox.h"

void SFilterObjectRowWidget::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, TSharedPtr<IFilterObject> InObject)
{
	Object = InObject;

	TSharedRef<SWrapBox> WrapBox = SNew(SWrapBox)
		.UseAllottedSize(true);

	TAttribute<bool> FilterEnabled = TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(Object.Get(), &IFilterObject::IsFilterEnabled));
	WrapBox->SetEnabled(FilterEnabled);
	
	Object->MakeWidget(WrapBox);

	STableRow<TSharedPtr<IFilterObject>>::Construct(
		STableRow<TSharedPtr<IFilterObject>>::FArguments()
		.Padding(FMargin(0.f, 2.f, 0.f, 2.f))
		.Content()
		[
			WrapBox
		], InOwnerTableView);
	
	TAttribute<FText> ToolTipText = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(Object.Get(), &IFilterObject::GetToolTipText));
	SetToolTipText(ToolTipText);
}

FReply SFilterObjectRowWidget::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		return FReply::Handled().BeginDragDrop(FFilterDragDropOp::New(Object->AsShared()));
	}

	return FReply::Unhandled();
}

void SFilterObjectRowWidget::OnDragEnter(FGeometry const& MyGeometry, FDragDropEvent const& DragDropEvent)
{
	Object->HandleDragEnter(DragDropEvent);
}

void SFilterObjectRowWidget::OnDragLeave(FDragDropEvent const& DragDropEvent)
{
	Object->HandleDragLeave(DragDropEvent);
}

FReply SFilterObjectRowWidget::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	return Object->HandleDrop(DragDropEvent);
}

FReply SFilterObjectRowWidget::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	return Object->HandleDragEnter(DragDropEvent);
}

