// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/SListView.h"

class IFilterObject;
class STableViewBase;

class SFilterObjectRowWidget : public STableRow<TSharedPtr<IFilterObject>>
{
	SLATE_BEGIN_ARGS(SFilterObjectRowWidget) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, TSharedPtr<IFilterObject> InObject);
	virtual ~SFilterObjectRowWidget() = default;

	/** Begin STableRow<TSharedPtr<IFilterObject>> overrides */
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnDragEnter(FGeometry const& MyGeometry, FDragDropEvent const& DragDropEvent) override;
	virtual void OnDragLeave(FDragDropEvent const& DragDropEvent) override;
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	/** End STableRow<TSharedPtr<IFilterObject>> overrides */

protected:	
	/** Filter object this row represents */
	TSharedPtr<IFilterObject> Object;	
};
