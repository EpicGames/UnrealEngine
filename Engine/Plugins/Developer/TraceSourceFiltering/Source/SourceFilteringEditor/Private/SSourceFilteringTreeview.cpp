// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSourceFilteringTreeview.h"

#include "Framework/Views/TableViewMetadata.h"
#include "ISessionSourceFilterService.h"
#include "FilterDragDropOperation.h"
#include "SUserTraceFilteringWidget.h"
#include "IFilterObject.h"

#define LOCTEXT_NAMESPACE "SSourceFilteringTreeView"

void SSourceFilteringTreeView::Construct(const FArguments& InArgs, TSharedRef<SUserTraceFilteringWidget> InOwner)
{
	Owner = InOwner;
	STreeView::Construct(InArgs);
}

int32 SSourceFilteringTreeView::GetNumItemsBeingObserved() const
{
	return STreeView::GetNumItemsBeingObserved() + 1;
}

FReply SSourceFilteringTreeView::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FFilterDragDropOp> FilterDragOp = DragDropEvent.GetOperationAs<FFilterDragDropOp>();

	if (FilterDragOp.IsValid())
	{
		if (FilterDragOp->FilterObject.IsValid() && !Owner.Pin()->FilterObjects.Contains(FilterDragOp->FilterObject))
		{
			FilterDragOp->SetIconText(FText::FromString(TEXT("\xf146")));
			FilterDragOp->SetText(LOCTEXT("RemoveFilterFromSetDragOpLabel", "Remove Filter from Filter Set"));
		}
	}

	return FReply::Unhandled();
}

void SSourceFilteringTreeView::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FFilterDragDropOp> FilterDragOp = DragDropEvent.GetOperationAs<FFilterDragDropOp>();

	if (FilterDragOp.IsValid())
	{
		if (FilterDragOp->FilterObject.IsValid() && !Owner.Pin()->FilterObjects.Contains(FilterDragOp->FilterObject))
		{
			FilterDragOp->SetIconText(FText::FromString(TEXT("\xf146")));
			FilterDragOp->SetText(LOCTEXT("RemoveFilterFromSetDragOpLabel", "Remove Filter from Filter Set"));
		}
	}
}

void SSourceFilteringTreeView::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FFilterDragDropOp> FilterDragOp = DragDropEvent.GetOperationAs<FFilterDragDropOp>();
	if (FilterDragOp.IsValid())
	{
		FilterDragOp->Reset();
	}
}

FReply SSourceFilteringTreeView::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FFilterDragDropOp> FilterDragOp = DragDropEvent.GetOperationAs<FFilterDragDropOp>();

	if (FilterDragOp.IsValid())
	{
		if (FilterDragOp->FilterObject.IsValid() && !Owner.Pin()->FilterObjects.Contains(FilterDragOp->FilterObject))
		{
			Owner.Pin()->SessionFilterService->MakeTopLevelFilter(FilterDragOp->FilterObject.Pin()->AsShared());
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE // "SSourceFilteringTreeView"
