// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/STreeView.h"

class SUserTraceFilteringWidget;
class IFilterObject;

/** Simple treeview derived widget , main purpose is to implement drag-and-drop capabilities */
class SSourceFilteringTreeView : public STreeView<TSharedPtr<IFilterObject>>
{
public:
	void Construct(const FArguments& InArgs, TSharedRef<SUserTraceFilteringWidget> InOwner);

protected:
	/** Begin STreeView<TSharedPtr<IFilterObject>> overrides */
	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual int32 GetNumItemsBeingObserved() const override;
	/** End STreeView<TSharedPtr<IFilterObject>> overrides */

protected:
	TWeakPtr<SUserTraceFilteringWidget> Owner;
};
