// Copyright Epic Games, Inc. All Rights Reserved.

#include "STraceSourceFilteringWidget.h"

#include "Framework/Commands/UICommandList.h"
#include "Widgets/Input/SComboButton.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ISessionSourceFilterService.h"
#include "Widgets/Images/SThrobber.h"
#include "SourceFilterService.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Modules/ModuleManager.h"
#include "TraceServices/ITraceServicesModule.h"

#include "Insights/IUnrealInsightsModule.h"
#include "Trace/StoreClient.h"

#include "SourceFilterStyle.h"
#include "TraceSourceFilteringSettings.h"

#include "SWorldTraceFilteringWidget.h"
#include "SClassTraceFilteringWidget.h"
#include "SUserTraceFilteringWidget.h"
#include "TraceServices/Model/AnalysisSession.h"

#define LOCTEXT_NAMESPACE "STraceSourceFilteringWidget"

STraceSourceFilteringWidget::~STraceSourceFilteringWidget()
{
	SaveFilteringSettings();
}

void STraceSourceFilteringWidget::Construct(const FArguments& InArgs)
{
	ITraceServicesModule& TraceServicesModule = FModuleManager::LoadModuleChecked<ITraceServicesModule>("TraceServices");

	ConstructMenuBox();
	
	ChildSlot
	[
		SNew(SBorder)
		.Padding(4.0f)
		.BorderImage(FSourceFilterStyle::GetBrush("SourceFilter.GroupBorder"))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(0.0f, 0.0f, 0.0f, 2.0f))
			[			
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					MenuBox->AsShared()
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SThrobber)
					.Visibility(this, &STraceSourceFilteringWidget::GetThrobberVisibility)
				]
			]
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[	
				SNew(SScrollBox)
				+ SScrollBox::Slot()
				[
					SAssignNew(ContentBox, SVerticalBox)
				]	
			]	
		]
	];

	AddExpandableArea(LOCTEXT("UserFilterHeaderText", "User Filters"), SAssignNew(UserFilterWidget, SUserTraceFilteringWidget));
	AddExpandableArea(LOCTEXT("ClassFilterHeaderText", "Class Filters"), SAssignNew(ClassFilterWidget, SClassTraceFilteringWidget));
	AddExpandableArea(LOCTEXT("WorldFilterHeaderText", "World Filters"), SAssignNew(WorldFilterWidget, SWorldTraceFilteringWidget));

	TAttribute<bool> EnabledAttribute = TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &STraceSourceFilteringWidget::ShouldWidgetsBeEnabled));
	MenuBox->SetEnabled(EnabledAttribute);
	ContentBox->SetEnabled(EnabledAttribute);
}

void STraceSourceFilteringWidget::ConstructMenuBox()
{
	SAssignNew(MenuBox, SHorizontalBox)
	+SHorizontalBox::Slot()
	.AutoWidth()
	.VAlign(VAlign_Center)
	.Padding(2.0f, 0.0f, 0.0f, 0.0f)
	[
		SNew(SComboButton)
		.Visibility(EVisibility::Visible)
		.ComboButtonStyle(FSourceFilterStyle::Get(), "SourceFilter.ComboButton")
		.ForegroundColor(FLinearColor::White)
		.ContentPadding(0.0f)
		.OnGetMenuContent(this, &STraceSourceFilteringWidget::OnGetOptionsMenu)
		.ButtonContent()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.TextStyle(FSourceFilterStyle::Get(), "SourceFilter.TextStyle")
				.Font(FSourceFilterStyle::Get().GetFontStyle("FontAwesome.9"))
				.Text(FText::FromString(FString(TEXT("\xf0b0"))) /*fa-filter*/)
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2, 0, 0, 0)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.TextStyle(FSourceFilterStyle::Get(), "SourceFilter.TextStyle")
				.Text(LOCTEXT("OptionMenuLabel", "Options"))
			]
		]
	];
}

void STraceSourceFilteringWidget::SetCurrentAnalysisSession(uint32 SessionHandle, TSharedRef<const TraceServices::IAnalysisSession> AnalysisSession)
{
	if (SessionFilterService.IsValid())
	{
		SessionFilterService->GetOnSessionStateChanged().RemoveAll(this);
}

	SessionFilterService = FSourceFilterService::GetFilterServiceForSession(SessionHandle, AnalysisSession);

	SessionFilterService->GetOnSessionStateChanged().AddSP(this, &STraceSourceFilteringWidget::RefreshFilteringData);

	WorldFilterWidget->SetSessionFilterService(SessionFilterService);
	ClassFilterWidget->SetSessionFilterService(SessionFilterService);
	UserFilterWidget->SetSessionFilterService(SessionFilterService);

	RefreshFilteringData();
}

bool STraceSourceFilteringWidget::HasValidFilterSession() const
{
	return SessionFilterService.IsValid();
}

EVisibility STraceSourceFilteringWidget::GetThrobberVisibility() const
{
	if (HasValidFilterSession())
	{
		return SessionFilterService->IsActionPending() ? EVisibility::Visible : EVisibility::Hidden;
	}

	return EVisibility::Hidden;
}

bool STraceSourceFilteringWidget::ShouldWidgetsBeEnabled() const
{
	if (HasValidFilterSession())
	{
		return !SessionFilterService->IsActionPending();
	}

	return false;
}

void STraceSourceFilteringWidget::RefreshFilteringData()
{
	FilteringSettings = SessionFilterService->GetFilterSettings();
}

void STraceSourceFilteringWidget::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (!SessionFilterService.IsValid() )
	{
		IUnrealInsightsModule& InsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
		TSharedPtr<const TraceServices::IAnalysisSession> AnalysisSession = InsightsModule.GetAnalysisSession();
		if (AnalysisSession.IsValid())
		{
			UE::Trace::FStoreClient* StoreClient = InsightsModule.GetStoreClient();

			if (StoreClient)
			{
				const UE::Trace::FStoreClient::FSessionInfo* SessionInfo = StoreClient->GetSessionInfoByTraceId(AnalysisSession->GetTraceId());
				if (SessionInfo && !AnalysisSession->IsAnalysisComplete())
				{
					SetCurrentAnalysisSession(SessionInfo->GetTraceId(), AnalysisSession.ToSharedRef());
				}
			}
		}
	}
}

TSharedRef<SWidget> STraceSourceFilteringWidget::OnGetOptionsMenu()
{
	FMenuBuilder Builder(true, TSharedPtr<FUICommandList>(), SessionFilterService->GetExtender());
	
	if (FilteringSettings)
	{
		Builder.BeginSection(NAME_None, LOCTEXT("VisualizationSectionLabel", "Visualization"));
		{
			Builder.AddSubMenu(LOCTEXT("VisualizeLabel", "Visualize"),
				LOCTEXT("DebugDrawingTooltip", "Sub menu related to Debug Drawing"),
				FNewMenuDelegate::CreateLambda([this](FMenuBuilder& InMenuBuilder)
					{
						InMenuBuilder.AddMenuEntry(
							LOCTEXT("DrawFilterStateLabel","Actor Filtering"), 
							LOCTEXT("DrawFilteringStateTooltip", "Draws the bounding box for each filter processed Actor in the world."),
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateLambda([this]() -> void
								{
									FilteringSettings->bDrawFilteringStates = !FilteringSettings->bDrawFilteringStates;
									SessionFilterService->UpdateFilterSettings(FilteringSettings);
								}),
								FCanExecuteAction::CreateLambda([this]() -> bool
								{
									return FilteringSettings != nullptr;
								}),
								FGetActionCheckState::CreateLambda([this]() -> ECheckBoxState
								{
									return FilteringSettings->bDrawFilteringStates ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
								})
							),
							NAME_None,
							EUserInterfaceActionType::ToggleButton
						);

						InMenuBuilder.AddMenuEntry(
							LOCTEXT("DrawFilterPassingOnlyLabel","Only Actor(s) Passing Filtering"), 
							LOCTEXT("DrawFilterPassingOnlyTooltip", "Only draws the filtering state for Actors that passsed the filtering state."),
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateLambda([this]()
								{
									FilteringSettings->bDrawOnlyPassingActors = !FilteringSettings->bDrawOnlyPassingActors;
									SessionFilterService->UpdateFilterSettings(FilteringSettings);
								}),
								FCanExecuteAction::CreateLambda([this]()
								{
									return FilteringSettings && FilteringSettings->bDrawFilteringStates;
								}),
								FGetActionCheckState::CreateLambda([this]()
								{
									return FilteringSettings->bDrawOnlyPassingActors ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
								})
							),
							NAME_None,
							EUserInterfaceActionType::ToggleButton
						);

						InMenuBuilder.AddMenuEntry(
							LOCTEXT("DrawNonPassingFiltersLabel","Draw Non Passing Filters"), 
							LOCTEXT("DrawNonPassingFiltersTooltip", "Draws the Filters that caused an Actor to be filtered out."),
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateLambda([this]()
								{
									FilteringSettings->bDrawFilterDescriptionForRejectedActors = !FilteringSettings->bDrawFilterDescriptionForRejectedActors;
									SessionFilterService->UpdateFilterSettings(FilteringSettings);
								}),
								FCanExecuteAction::CreateLambda([this]()
								{
									return FilteringSettings && FilteringSettings->bDrawFilteringStates;
								}),
								FGetActionCheckState::CreateLambda([this]()
								{
									return FilteringSettings->bDrawFilterDescriptionForRejectedActors ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
								})
							),
							NAME_None,
							EUserInterfaceActionType::ToggleButton
						);
					
						InMenuBuilder.AddMenuEntry(
							LOCTEXT("OutputFilteringStateLabel","Output (optimized) Filtering State"), 
							LOCTEXT("OutputFilteringStateTooltip", "Whether or not to output the filtering state information, whenever it changes and the optimized data is rebuild."),
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateLambda([this]()
								{
									FilteringSettings->bOutputOptimizedFilterState = !FilteringSettings->bOutputOptimizedFilterState;
									SessionFilterService->UpdateFilterSettings(FilteringSettings);
								}),
								FCanExecuteAction::CreateLambda([this]()
								{
									return FilteringSettings != nullptr;
								}),
								FGetActionCheckState::CreateLambda([this]()
								{
									return FilteringSettings->bOutputOptimizedFilterState ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
								})
							),
							NAME_None,
							EUserInterfaceActionType::ToggleButton
						);
					
					}
				),
				false,
				FSlateIcon(),
				false
			);
		}
		Builder.EndSection();
	}
	
	const FName SectionName = TEXT("FilterOptionsMenu");
	Builder.BeginSection(SectionName, LOCTEXT("FiltersSectionLabel", "Filters"));
	{
		Builder.AddMenuEntry(
			LOCTEXT("ResetFiltersLabel","Reset Filters"), 
			LOCTEXT("ResetFiltersTooltip", "Removes all currently set filters."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this]()
				{
					SessionFilterService->ResetFilters();
				}),
				FCanExecuteAction::CreateLambda([this]()
				{
					return SessionFilterService.IsValid();
				})
			)
		);	
	}
	Builder.EndSection();

	return Builder.MakeWidget();	
}

void STraceSourceFilteringWidget::SaveFilteringSettings()
{
	if (FilteringSettings)
	{
		FilteringSettings->SaveConfig();
	}
}

void STraceSourceFilteringWidget::AddExpandableArea(const FText& AreaName, TSharedRef<SWidget> AreaWidget)
{
	ContentBox->AddSlot()
	.AutoHeight()
	.Padding(2.0f)
	[
		SNew(SExpandableArea)
		.HeaderPadding(FMargin(2.0f))
		.Padding(FMargin(10.f))
		.BorderImage(FSourceFilterStyle::Get().GetBrush("ExpandableAreaBrush"))
		.BorderBackgroundColor(FLinearColor(.6f, .6f, .6f, 1.0f))
		.BodyBorderBackgroundColor(FLinearColor::Transparent)
		.BodyContent()
		[
			AreaWidget
		]
		.HeaderContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(AreaName)
				.Justification(ETextJustify::Left)
				.TextStyle(FSourceFilterStyle::Get(), "SourceFilter.TextStyle")
			]
		]
	];
}

#undef LOCTEXT_NAMESPACE // "STraceSourceFilteringWidget"
