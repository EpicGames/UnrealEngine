// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Widgets/SCompoundWidget.h"

class ISessionSourceFilterService;
class SVerticalBox;
class UTraceSourceFilteringSettings;
namespace TraceServices { class IAnalysisSession; }

#if WITH_EDITOR
#endif // WITH_EDITOR

class SWorldTraceFilteringWidget;
class SClassTraceFilteringWidget;
class SUserTraceFilteringWidget;
class SComboButton;
class SHorizontalBox;

class STraceSourceFilteringWidget : public SCompoundWidget
{
public:
	/** Default constructor. */
	STraceSourceFilteringWidget() : FilteringSettings(nullptr) {}

	/** Virtual destructor. */
	virtual ~STraceSourceFilteringWidget();

	SLATE_BEGIN_ARGS(STraceSourceFilteringWidget) {}
	SLATE_END_ARGS()

	/** Constructs this widget. */
	void Construct(const FArguments& InArgs);

	/** Begin SCompoundWidget overrides */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	/** End SCompoundWidget overrides */
	
protected:
	void ConstructMenuBox();

	TSharedRef<SWidget> OnGetOptionsMenu();

	/** Callback for whenever a different analysis session (store) has been retrieved */
	void SetCurrentAnalysisSession(uint32 SessionHandle, TSharedRef<const TraceServices::IAnalysisSession> AnalysisSession);

	/** Returns whether or not a valid ISessionSourceFilterService is available / set */
	bool HasValidFilterSession() const;

	/** Returns visibility state for SThrobber, used to indicate pending filter session request */
	EVisibility GetThrobberVisibility() const;

	/** Returns whether or not the contained widgets should be enabled, determined by having a valid session and no pending request */
	bool ShouldWidgetsBeEnabled() const;

	/** Refreshes the filtering data and state, using SessionFilterService, represented by this widget */
	void RefreshFilteringData();
	
	/** Save current UTraceSourceFilteringSettings state to INI files */
	void SaveFilteringSettings();

	/** Adds an expandable area widget to ContentBox */
	void AddExpandableArea(const FText& AreaName, TSharedRef<SWidget> AreaWidget);
protected:
	/** Slate widget containing the Add Filter and Options widgets, used for disabling/enabling according to the session state */
	TSharedPtr<SHorizontalBox> MenuBox;

	/** Vertical box contain the individual expandable areas */
	TSharedPtr<SVerticalBox> ContentBox;
	
	/** Filter session instance, used to retrieve data and communicate with connected application */
	TSharedPtr<ISessionSourceFilterService> SessionFilterService;

	/** Widget used for filtering UWorld's traceability on the connected filter session */
	TSharedPtr<SWorldTraceFilteringWidget> WorldFilterWidget;

	/** Widget used for filtering AActors when applying user filters for a given UWorld */
	TSharedPtr<SClassTraceFilteringWidget> ClassFilterWidget;
	/** Widget used for setting up User Filters applied to actor within a given UWorld */
	TSharedPtr<SUserTraceFilteringWidget> UserFilterWidget;
	   
	/** Timestamp at which the treeview data was last retrieved from SessionFilterService */
	FDateTime SyncTimestamp;

	/** Cached pointer to Filtering Settings retrieved from SessionFilterService */
	UTraceSourceFilteringSettings* FilteringSettings;
};
