// Copyright Epic Games, Inc. All Rights Reserved.

#include "SUserTraceFilteringWidget.h"


#include "DataSourceFiltering.h"
#include "Framework/Application/SlateApplication.h"
#include "DetailsViewArgs.h"
#include "SourceFilterStyle.h"
#include "Framework/Commands/UICommandList.h"
#include "SSourceFilteringTreeview.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SFilterObjectWidget.h"
#include "IDataSourceFilterSetInterface.h"
#include "TreeViewBuilder.h"
#include "ISessionSourceFilterService.h"
#include "IFilterObject.h"
#include "Widgets/Input/SComboButton.h"

#if WITH_EDITOR
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "IDetailsView.h"
#include "Engine/Blueprint.h"
#include "EmptySourceFilter.h"
#include "Editor.h"
#include "Subsystems/AssetEditorSubsystem.h"
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "SUserTraceFilteringWidget"

void SUserTraceFilteringWidget::Construct(const FArguments& InArgs)
{
#if WITH_EDITOR
	ConstructInstanceDetailsView();
#endif // WITH_EDITOR

	ConstructTreeview();
	ConstructUserFilterPickerButton();

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 0.f, 0.f, 2.f)
		[
			FilterTreeView->AsShared()
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.VAlign(VAlign_Center)
#if WITH_EDITOR
		.Padding(0.0f, 0.f, 0.f, 2.f)
#endif // WITH_EDITOR
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(10.0f, 2.f, 0.f, 0.f)
			[
				AddUserFilterButton->AsShared()
			]
		]
#if WITH_EDITOR
		+ SVerticalBox::Slot()
		.AutoHeight()							
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(10.0f, 0.f, 0.f, 0.f)
			[
				FilterInstanceDetailsView->AsShared()
			]
		]						
#endif // WITH_EDITOR
	];

	TAttribute<bool> EnabledAttribute = TAttribute<bool>::Create([this]() -> bool
	{
		return SessionFilterService.IsValid() && !SessionFilterService->IsActionPending();
	});

	FilterTreeView->SetEnabled(EnabledAttribute);
#if WITH_EDITOR
	FilterInstanceDetailsView->SetEnabled(EnabledAttribute);
#endif // WITH_EDITOR
}

void SUserTraceFilteringWidget::SetSessionFilterService(TSharedPtr<ISessionSourceFilterService> InSessionFilterService)
{
	if (SessionFilterService.IsValid())
	{
		SessionFilterService->GetOnSessionStateChanged().RemoveAll(this);
	}

	SessionFilterService = InSessionFilterService;

	SessionFilterService->GetOnSessionStateChanged().AddSP(this, &SUserTraceFilteringWidget::RefreshUserFilterData);

	RefreshUserFilterData();
}

void SUserTraceFilteringWidget::ConstructTreeview()
{
	SAssignNew(FilterTreeView, SSourceFilteringTreeView, StaticCastSharedRef<SUserTraceFilteringWidget>(AsShared()))
	.TreeItemsSource(&FilterObjects)
	.OnGetChildren_Lambda([this](TSharedPtr<IFilterObject> InObject, TArray<TSharedPtr<IFilterObject>>& OutChildren)
	{
		if (TArray<TSharedPtr<IFilterObject>> * ChildArray = ParentToChildren.Find(InObject))
		{
			OutChildren.Append(*ChildArray);
		}
	})
	.OnGenerateRow_Lambda([](TSharedPtr<IFilterObject> InItem, const TSharedRef<STableViewBase>& OwnerTable)
	{
		return SNew(SFilterObjectRowWidget, OwnerTable, InItem);
	})
#if WITH_EDITOR
	.OnSelectionChanged_Lambda([this](TSharedPtr<IFilterObject> InItem, ESelectInfo::Type SelectInfo) -> void
	{
		if (InItem.IsValid())
		{
			UObject* Filter = InItem->GetFilter();
			FilterInstanceDetailsView->SetObject(Filter);
		}
		else
		{
			FilterInstanceDetailsView->SetObject(nullptr);
		}	
	})
#endif // WITH_EDITOR
	.OnContextMenuOpening(this, &SUserTraceFilteringWidget::OnContextMenuOpening);
}

#if WITH_EDITOR
void SUserTraceFilteringWidget::ConstructInstanceDetailsView()
{
	// Create a property view
	FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Automatic;
	DetailsViewArgs.bShowOptions = false;	

	FilterInstanceDetailsView = EditModule.CreateDetailView(DetailsViewArgs);
}
#endif // WITH_EDITOR

void SUserTraceFilteringWidget::ConstructUserFilterPickerButton()
{
	/** Callback for whenever a Filter class (name) was selected */
	auto OnActorFilterClassPicked = [this](FString PickedFilterName)
	{
		if (SessionFilterService.Get())
		{
			SessionFilterService->AddFilter(PickedFilterName);
			AddUserFilterButton->SetIsOpen(false);
		}
	};

	SAssignNew(AddUserFilterButton, SComboButton)
	.Visibility(EVisibility::Visible)
	.ComboButtonStyle(FSourceFilterStyle::Get(), "SourceFilter.ComboButton")
	.ForegroundColor(FLinearColor::White)
	.ContentPadding(FMargin(2.f, 2.0f))
	.HasDownArrow(false)
	.OnGetMenuContent(FOnGetContent::CreateLambda([OnActorFilterClassPicked, this]()
	{
		FMenuBuilder MenuBuilder(true, TSharedPtr<FUICommandList>(), SessionFilterService->GetExtender());

		MenuBuilder.BeginSection(FName("FilterPicker"));
		{
			MenuBuilder.AddWidget(SessionFilterService->GetFilterPickerWidget(FOnFilterClassPicked::CreateLambda(OnActorFilterClassPicked)), FText::GetEmpty(), true, false);
		}
		MenuBuilder.EndSection();

		return MenuBuilder.MakeWidget();
	}))
	.ButtonContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.TextStyle(FSourceFilterStyle::Get(), "SourceFilter.TextStyle")
		.Font(FSourceFilterStyle::Get().GetFontStyle("FontAwesome.12"))
		.Text(FText::FromString(FString(TEXT("\xf0fe"))) /*fa-filter*/)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2, 0, 0, 0)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.TextStyle(FSourceFilterStyle::Get(), "SourceFilter.TextStyle")
			.Text(LOCTEXT("FilterMenuLabel", "Add Filter"))
		]
	];
}

TSharedPtr<SWidget> SUserTraceFilteringWidget::OnContextMenuOpening()
{
	/** Selection information */
	TArray<TSharedPtr<IFilterObject>> FilterSelection;
	FilterTreeView->GetSelectedItems(FilterSelection);

	FMenuBuilder MenuBuilder(true, TSharedPtr<const FUICommandList>(), SessionFilterService->GetExtender());

	if (FilterSelection.Num() > 0)
	{
		bool bSelectionContainsFilterSet = false;
		bool bSelectionContainsBPFilter = false;
		bool bSelectionContainsEmptyFilter = false;
		bool bSelectionContainsNonEmptyFilter = false;
		const bool bMultiSelection = FilterSelection.Num() > 1;

		/** Gather information about current filter selection set */
		for (const TSharedPtr<IFilterObject>& Filter : FilterSelection)
		{
			const UObject* FilterObject = Filter->GetFilter();
			if (ensure(FilterObject))
			{
				if (const IDataSourceFilterSetInterface* SetInterface = Cast<IDataSourceFilterSetInterface>(FilterObject))
				{
					bSelectionContainsFilterSet = true;
				}

#if WITH_EDITOR
				if (const UEmptySourceFilter* EmptyFilter = Cast<UEmptySourceFilter>(FilterObject))
				{
					bSelectionContainsEmptyFilter = true;
				}
				else
				{
					bSelectionContainsNonEmptyFilter = true;
				}
#endif // WITH_EDITOR

				if (FilterObject->GetClass()->ClassGeneratedBy != nullptr)
				{
					bSelectionContainsBPFilter = true;
				}
			}
		}

#if WITH_EDITOR
		if (bSelectionContainsBPFilter)
		{
			MenuBuilder.BeginSection(NAME_None, LOCTEXT("BlueprintFilterSectionLabel", "Blueprint Filter"));
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("OpenFilterLabel", "Open Filter Blueprint"),
					LOCTEXT("OpenFilterTooltip", "Opens this Filter's Blueprint."),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.Blueprint"),
					FUIAction(
						FExecuteAction::CreateLambda([FilterSelection]()
						{
							for (const TSharedPtr<IFilterObject>& FilterObject : FilterSelection)
							{
								const UObject* FilterUObject = FilterObject->GetFilter();
								if (ensure(FilterUObject))
								{
									if (UBlueprint* Blueprint = Cast<UBlueprint>(FilterUObject->GetClass()->ClassGeneratedBy))
									{
										GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Blueprint);
									}
								}
							}
						})
					)
				);
			}
			MenuBuilder.EndSection();

		}
#endif // WITH_EDITOR

		/** Single selection of a filter set */
		if (bSelectionContainsFilterSet && !bMultiSelection)
		{
			auto AddFilterToSet = [this, FilterSelection](FString ClassName)
			{
				if (SessionFilterService.Get())
				{
					SessionFilterService->AddFilterToSet(FilterSelection[0]->AsShared(), ClassName);
				}

				FSlateApplication::Get().DismissAllMenus();
			};

			MenuBuilder.BeginSection(NAME_None, LOCTEXT("FilterSetContextMenuLabel", "Filter Set"));
			{
				MenuBuilder.AddSubMenu(
					LOCTEXT("AddFilterToSetLabel", "Add Filter"),
					LOCTEXT("AddFilterToSetTooltip", "Adds a filter to this Filtering Set."),
					FNewMenuDelegate::CreateLambda([this, AddFilterToSet](FMenuBuilder& InSubMenuBuilder)
				{
					InSubMenuBuilder.AddWidget(
						SessionFilterService->GetFilterPickerWidget(FOnFilterClassPicked::CreateLambda(AddFilterToSet)),
						FText::GetEmpty(),
						true
					);
				})
				);
			}
			MenuBuilder.EndSection();
		}

		MenuBuilder.BeginSection(NAME_None, LOCTEXT("FiltersContextMenuLabel", "Filter"));
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("EnableFilterLabel", "Filter Enabled"),
				LOCTEXT("ToggleFilterTooltips", "Sets whether or not this Filter should be considered when applying the set of filters"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([FilterSelection, this]()
			{
				bool bEnabled = false;
				bool bDisabled = false;

				for (const TSharedPtr<IFilterObject>& FilterObject : FilterSelection)
				{
					bEnabled |= FilterObject->IsFilterEnabled();
					bDisabled |= !FilterObject->IsFilterEnabled();
				}

				for (const TSharedPtr<IFilterObject>& FilterObject : FilterSelection)
				{
					SessionFilterService->SetFilterState(FilterObject->AsShared(), (bEnabled && bDisabled) || bDisabled);
				}
			}),
					FCanExecuteAction(),
				FGetActionCheckState::CreateLambda([FilterSelection]()
			{
				bool bEnabled = false;
				bool bDisabled = false;

				for (const TSharedPtr<IFilterObject>& Filter : FilterSelection)
				{
					bEnabled |= Filter->IsFilterEnabled();
					bDisabled |= !Filter->IsFilterEnabled();
				}

				return bEnabled ? (bDisabled ? ECheckBoxState::Undetermined : ECheckBoxState::Checked) : ECheckBoxState::Unchecked;
			})
				),
				NAME_None,
				EUserInterfaceActionType::Check
				);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("RemoveFilterLabel", "Remove Filter"),
				LOCTEXT("RemoveFilterTooltip", "Removes this Filter from the filtering set."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([this, FilterSelection]()
			{
				for (const TSharedPtr<IFilterObject>& FilterObject : FilterSelection)
				{
					SessionFilterService->RemoveFilter(FilterObject->AsShared());
				}
			})
				)
			);
		}
		MenuBuilder.EndSection();

		/** Single selection of a valid Filter instance */
		if (!bMultiSelection && bSelectionContainsNonEmptyFilter)
		{
			MenuBuilder.BeginSection(NAME_None, LOCTEXT("AddFilterSetSectionLabel", "Add Filter Set"));
			{
				FText LabelTextFormat = LOCTEXT("MakeFilterSetLabel", "{0}");
				FText ToolTipTextFormat = LOCTEXT("MakeFilterSetTooltip", "Creates a new filter set, containing this filter, with the {0} operator");

				const UEnum* EnumPtr = FindObject<UEnum>(nullptr, TEXT("/Script/SourceFilteringCore.EFilterSetMode"), EFindObjectFlags::ExactClass);
				for (EFilterSetMode Mode : TEnumRange<EFilterSetMode>())
				{
					FText ModeText = EnumPtr->GetDisplayNameTextByValue((int64)Mode);
					MenuBuilder.AddMenuEntry(
						FText::Format(LabelTextFormat, ModeText),
						FText::Format(ToolTipTextFormat, ModeText),
						FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateLambda([this, Mode, FilterSelection]()
					{
						SessionFilterService->MakeFilterSet(FilterSelection[0]->AsShared(), Mode);
					})
						)
					);
				}
			}
			MenuBuilder.EndSection();
		}
	}
	else
	{
		auto OnFilterClassPicked = [this](FString PickedFilterName)
		{
			if (SessionFilterService.Get())
			{
				SessionFilterService->AddFilter(PickedFilterName);
				AddUserFilterButton->SetIsOpen(false);
			}
		};

		MenuBuilder.AddWidget(SessionFilterService->GetFilterPickerWidget(FOnFilterClassPicked::CreateLambda(OnFilterClassPicked)), FText::GetEmpty(), true, false);
	}

	return MenuBuilder.MakeWidget();
}

void SUserTraceFilteringWidget::RefreshUserFilterData()
{
	SaveTreeviewState();

	FilterObjects.Empty();
	ParentToChildren.Empty();
	FlatFilterObjects.Empty();

	FTreeViewDataBuilder Builder(FilterObjects, ParentToChildren, FlatFilterObjects);
	SessionFilterService->PopulateTreeView(Builder);

	FilterTreeView->RequestTreeRefresh();

	RestoreTreeviewState();
}


void SUserTraceFilteringWidget::SaveTreeviewState()
{
	if (FilterTreeView)
	{
		ensure(ExpandedFilters.Num() == 0);
		TSet<TSharedPtr<IFilterObject>> TreeviewExpandedObjects;
		FilterTreeView->GetExpandedItems(TreeviewExpandedObjects);
		Algo::Transform(TreeviewExpandedObjects, ExpandedFilters, [](TSharedPtr<IFilterObject> Object)
		{
			return GetTypeHash(Object->GetFilter());
		});

		ensure(SelectedFilters.Num() == 0);
		TArray<TSharedPtr<IFilterObject>> TreeviewSelectedObjects;
		FilterTreeView->GetSelectedItems(TreeviewSelectedObjects);
		Algo::Transform(TreeviewSelectedObjects, SelectedFilters, [](TSharedPtr<IFilterObject> Object)
		{
			return GetTypeHash(Object->GetFilter());
		});

	}
}

void SUserTraceFilteringWidget::RestoreTreeviewState()
{
	if (FilterTreeView)
	{
		FilterTreeView->ClearExpandedItems();
		for (TSharedPtr<IFilterObject> FilterObject : FlatFilterObjects)
		{
			if (ExpandedFilters.Contains(GetTypeHash(FilterObject->GetFilter())))
			{
				FilterTreeView->SetItemExpansion(FilterObject, true);
			}
		}

		ExpandedFilters.Empty();

		FilterTreeView->ClearSelection();
		for (TSharedPtr<IFilterObject> FilterObject : FlatFilterObjects)
		{
			if (SelectedFilters.Contains(GetTypeHash(FilterObject->GetFilter())))
			{
				FilterTreeView->SetItemSelection(FilterObject, true);
			}
		}

		SelectedFilters.Empty();
	}
}

#undef LOCTEXT_NAMESPACE // "SUserTraceFilteringWidget"
