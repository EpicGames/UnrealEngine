// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class ISessionSourceFilterService;
class IFilterObject;

class SWrapBox;
class SComboButton;
class SSourceFilteringTreeView;
class IDetailsView;

class SUserTraceFilteringWidget : public SCompoundWidget
{
	friend class SSourceFilteringTreeView;
public:
	/** Default constructor. */
	SUserTraceFilteringWidget() {}
	virtual ~SUserTraceFilteringWidget() = default;

	SLATE_BEGIN_ARGS(SUserTraceFilteringWidget) {}
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs);
	
	void SetSessionFilterService(TSharedPtr<ISessionSourceFilterService> InSessionFilterService);
protected:
	void ConstructTreeview();
	void ConstructInstanceDetailsView();
	void ConstructUserFilterPickerButton();

	TSharedPtr<SWidget> OnContextMenuOpening();

	void RefreshUserFilterData();

	/** Saves and restores the current selection and expansion state of FilterTreeView*/
	void SaveTreeviewState();
	void RestoreTreeviewState();
protected:
	/** Session service used to retrieve state and request filtering changes */
	TSharedPtr<ISessionSourceFilterService> SessionFilterService;

	/** Slate widget used to add filter instances to the session */
	TSharedPtr<SComboButton> AddUserFilterButton;
	
	/** Data used to populate the Filter Treeview */
	TArray<TSharedPtr<IFilterObject>> FilterObjects;
	TMap<TSharedPtr<IFilterObject>, TArray<TSharedPtr<IFilterObject>>> ParentToChildren;
	TArray<TSharedPtr<IFilterObject>> FlatFilterObjects;

	/** Used to store hash values of, treeview expanded and selected, filter instances when refreshing the treeview data */
	TArray<int32> ExpandedFilters;
	TArray<int32> SelectedFilters;


	/** Treeview used to display all currently represented Filters */
	TSharedPtr<SSourceFilteringTreeView> FilterTreeView;

#if WITH_EDITOR	
	/** Details view, used for displaying selected Filter UProperties */
	TSharedPtr<IDetailsView> FilterInstanceDetailsView;
#endif // WITH_EDITOR
};
