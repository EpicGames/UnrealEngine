// Copyright Epic Games, Inc. All Rights Reserved.

#include "SWorldObjectWidget.h"
#include "Widgets/Input/SCheckBox.h"
#include "WorldObject.h"

#include "ISessionSourceFilterService.h"

void SWorldObjectRowWidget::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, TSharedPtr<FWorldObject> InWorldObject, TSharedPtr<ISessionSourceFilterService> InFilterService)
{
	WorldObject = InWorldObject;
	FilterService = InFilterService;

	STableRow<TSharedPtr<FWorldObject>>::Construct(
		STableRow<TSharedPtr<FWorldObject>>::FArguments()
		.Padding(0.0f)
		.Content()
		[
			SNew(SHorizontalBox)
			// Name 
			+ SHorizontalBox::Slot()
			.FillWidth(0.5f)
			.Padding(2, 0, 0, 0)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(WorldObject->GetDisplayText())
			]
			// Traceability checkbox 
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0, 0, 2.f, 0)
			[
				/** Called when the checked state has changed */
				SNew(SCheckBox)
				.IsChecked_Lambda([this]() -> ECheckBoxState
				{
					return WorldObject->CanOutputData() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
				.OnCheckStateChanged_Lambda([this](ECheckBoxState State)
				{
					FilterService->SetWorldTraceability(WorldObject->AsShared(), State == ECheckBoxState::Checked);
				})
			]
		],
		InOwnerTableView
	);
}
