// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/SListView.h"

class FWorldObject;
class STableViewBase;
class ISessionSourceFilterService;
class FWorldObject;

/** Table row representing a UWorld instance, as a FWorldObject, within SWorldFilterWidget */
class SWorldObjectRowWidget : public STableRow<TSharedPtr<FWorldObject>>
{
	SLATE_BEGIN_ARGS(SWorldObjectRowWidget) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, TSharedPtr<FWorldObject> InWorldObject, TSharedPtr<ISessionSourceFilterService> InFilterService);
	virtual ~SWorldObjectRowWidget() = default;

protected:
	/* The World object that this tree item represents */
	TSharedPtr<FWorldObject> WorldObject;

	/** Session service that has generated this object */
	TSharedPtr<ISessionSourceFilterService> FilterService;
};
