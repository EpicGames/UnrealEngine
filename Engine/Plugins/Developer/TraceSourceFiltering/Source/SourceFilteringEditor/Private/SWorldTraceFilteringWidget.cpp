// Copyright Epic Games, Inc. All Rights Reserved.

#include "SWorldTraceFilteringWidget.h"


#include "Framework/Views/TableViewMetadata.h"
#include "SWorldObjectWidget.h"
#include "SourceFilterStyle.h"
#include "ISessionSourceFilterService.h"

#define LOCTEXT_NAMESPACE "SWorldFilterWidget"

void SWorldTraceFilteringWidget::Construct(const FArguments& InArgs)
{
	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SAssignNew(FiltersContainerBox, SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(0.0f, 0.0f, 4.0f, 2.0f))
			[
				SAssignNew(FiltersLabelBox, SVerticalBox)
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SAssignNew(FiltersWidgetBox, SVerticalBox)
			]
		]
		+ SVerticalBox::Slot()
		.Padding(FMargin(0.0f, 0.0f, 0.0f, 2.0f))
		.AutoHeight()
		[
			SNew(STextBlock)
			.TextStyle(FSourceFilterStyle::Get(), "SourceFilter.TextStyle")
			.Text(LOCTEXT("WorldsTreeviewLabel", "Worlds:"))
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		[	
			SAssignNew(WorldListView, SListView<TSharedPtr<FWorldObject>>)
			.ListItemsSource(&WorldObjects)
			.OnGenerateRow_Lambda([this](TSharedPtr<FWorldObject> InItem, const TSharedRef<STableViewBase>& OwnerTable) -> TSharedRef<ITableRow>
			{
				return SNew(SWorldObjectRowWidget, OwnerTable, InItem, SessionFilterService);
			})
		]	
	];

	TAttribute<bool> EnabledAttribute = TAttribute<bool>::Create([this]() -> bool
	{
		return SessionFilterService.IsValid() && !SessionFilterService->IsActionPending();
	});

	FiltersContainerBox->SetEnabled(EnabledAttribute);
	WorldListView->SetEnabled(EnabledAttribute);
}

void SWorldTraceFilteringWidget::SetSessionFilterService(TSharedPtr<ISessionSourceFilterService> InSessionFilterService)
{
	if (SessionFilterService.IsValid())
	{
		SessionFilterService->GetOnSessionStateChanged().RemoveAll(this);
	}
	SessionFilterService = InSessionFilterService;
	SessionFilterService->GetOnSessionStateChanged().AddSP(this, &SWorldTraceFilteringWidget::RefreshWorldData);
	RefreshWorldData();
}

void SWorldTraceFilteringWidget::RefreshWorldData()
{
	WorldObjects.Empty();
	SessionFilterService->GetWorldObjects(WorldObjects);
	WorldListView->RequestListRefresh();

	FiltersLabelBox->ClearChildren();
	FiltersWidgetBox->ClearChildren();

	const TArray<TSharedPtr<IWorldTraceFilter>>& WorldFilters = SessionFilterService->GetWorldFilters();
	if (WorldFilters.Num())
	{
		for (TSharedPtr<IWorldTraceFilter> WorldFilter : WorldFilters)
		{
			FiltersLabelBox->AddSlot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.f, 4.f, 0.f, 8.f)
				[
					SNew(STextBlock)
					.TextStyle(FSourceFilterStyle::Get(), "SourceFilter.TextStyle")
					.Text(FText::Format(LOCTEXT("WorldFilterLabel", "{0}:"), WorldFilter->GetDisplayText()))
				]
			];

			FiltersWidgetBox->AddSlot()
			.AutoHeight()
			.Padding(0.f, 0.f, 0.f, 4.f)
			[
				WorldFilter->GenerateWidget()
			];
		}
	}
}

#undef LOCTEXT_NAMESPACE // "SWorldFilterWidget"
