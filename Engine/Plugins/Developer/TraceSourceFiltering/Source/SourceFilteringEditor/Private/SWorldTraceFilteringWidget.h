// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class SHorizontalBox;
class SVerticalBox;
template <typename ItemType> class SListView;

class SComboButton;
class ISessionSourceFilterService;
class FWorldObject;

class SWorldTraceFilteringWidget : public SCompoundWidget
{
public:
	/** Default constructor. */
	SWorldTraceFilteringWidget() {}

	virtual ~SWorldTraceFilteringWidget() = default;

	SLATE_BEGIN_ARGS(SWorldTraceFilteringWidget) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	void SetSessionFilterService(TSharedPtr<ISessionSourceFilterService> InSessionFilterService);
protected:
	void RefreshWorldData();
protected:
	/** Session service used to retrieve state and request filtering changes */
	TSharedPtr<ISessionSourceFilterService> SessionFilterService;

	/** Latest set of retrieve World Objects from SessionFilterService */
	TArray<TSharedPtr<FWorldObject>> WorldObjects;

	/** Listview widget used to display WorldObjects  */
	TSharedPtr<SListView<TSharedPtr<FWorldObject>>> WorldListView;

	TSharedPtr<SHorizontalBox> FiltersContainerBox;
	TSharedPtr<SVerticalBox> FiltersLabelBox;
	TSharedPtr<SVerticalBox> FiltersWidgetBox;
};
