// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceFilterService.h"



#if WITH_EDITOR
#include "EditorSessionSourceFilterService.h"
#endif

TSharedRef<ISessionSourceFilterService> FSourceFilterService::GetFilterServiceForSession(uint32 InHandle, TSharedRef<const TraceServices::IAnalysisSession> AnalysisSession)
{
	TSharedPtr<ISessionSourceFilterService> NewService = nullptr;		
	NewService = MakeShareable(new FEditorSessionSourceFilterService());
	return NewService->AsShared();
}
