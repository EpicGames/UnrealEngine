// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class ISessionSourceFilterService;

namespace TraceServices
{
	class IAnalysisSession;
}

class FSourceFilterService
{
public:
	/** Retrieves the ISessionSourceFilterService for the provided Trace Session Handle */
	static TSharedRef<ISessionSourceFilterService> GetFilterServiceForSession(uint32 InHandle, TSharedRef<const TraceServices::IAnalysisSession> AnalysisSession);
};
