// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceFilterStyle.h"
#include "Brushes/SlateBoxBrush.h"
#include "Styling/SlateStyleRegistry.h"
#include "Brushes/SlateColorBrush.h"
#include "Styling/SlateTypes.h"
#include "Brushes/SlateImageBrush.h"
#include "Styling/CoreStyle.h"
#include "Brushes/SlateNoResource.h"
#include "Styling/SlateStyleMacros.h"
#include "Misc/Paths.h"

// This is to fix the issue that SlateStyleMacros like IMAGE_BRUSH look for RootToContentDir but StyleSet->RootToContentDir is how this style is set up
#define RootToContentDir StyleSet->RootToContentDir

FSlateBrush* FSourceFilterStyle::FilterBrush = nullptr;
FSlateBrush* FSourceFilterStyle::FilterSetBrush = nullptr;

TSharedPtr< FSlateStyleSet > FSourceFilterStyle::StyleSet = nullptr;

FTextBlockStyle FSourceFilterStyle::NormalText;

// Const icon sizes
static const FVector2D Icon8x8(8.0f, 8.0f);
static const FVector2D Icon9x19(9.0f, 19.0f);
static const FVector2D Icon14x14(14.0f, 14.0f);
static const FVector2D Icon16x16(16.0f, 16.0f);
static const FVector2D Icon20x20(20.0f, 20.0f);
static const FVector2D Icon22x22(22.0f, 22.0f);
static const FVector2D Icon24x24(24.0f, 24.0f);
static const FVector2D Icon28x28(28.0f, 28.0f);
static const FVector2D Icon27x31(27.0f, 31.0f);
static const FVector2D Icon26x26(26.0f, 26.0f);
static const FVector2D Icon32x32(32.0f, 32.0f);
static const FVector2D Icon40x40(40.0f, 40.0f);
static const FVector2D Icon48x48(48.0f, 48.0f);
static const FVector2D Icon75x82(75.0f, 82.0f);
static const FVector2D Icon360x32(360.0f, 32.0f);
static const FVector2D Icon171x39(171.0f, 39.0f);
static const FVector2D Icon170x50(170.0f, 50.0f);
static const FVector2D Icon267x140(170.0f, 50.0f);

void FSourceFilterStyle::Initialize()
{
	// Only register once
	if( StyleSet.IsValid() )
	{
		return;
	}

	StyleSet = MakeShareable(new FSlateStyleSet("SourceFilter") );
	
	StyleSet->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	StyleSet->Set("SourceFilter.GroupBorder", new BOX_BRUSH("Common/GroupBorder", FMargin(4.0f / 16.0f)));


	NormalText = FTextBlockStyle()
		.SetFont(DEFAULT_FONT("Regular", FCoreStyle::RegularTextSize))
		.SetColorAndOpacity(FSlateColor::UseForeground())
		.SetShadowOffset(FVector2D::ZeroVector)
		.SetShadowColorAndOpacity(FLinearColor::Black)
		.SetHighlightColor(FLinearColor(0.02f, 0.3f, 0.0f))
		.SetHighlightShape(BOX_BRUSH("Common/TextBlockHighlightShape", FMargin(3.f / 8.f)));


	FSourceFilterStyle::FilterBrush = new BOX_BRUSH("Common/FlatButton", 2.0f / 8.0f, FLinearColor(0.9f, 0.9f, 0.9f, 0.9f));
	StyleSet->Set("SourceFilter.FilterBrush", FSourceFilterStyle::FilterBrush);
	
	FSourceFilterStyle::FilterSetBrush = new BOX_BRUSH("Common/FlatButton", 2.0f / 8.0f, FLinearColor(0.125f, 0.125f, 0.125f, 0.9f));
	StyleSet->Set("SourceFilter.FilterSetBrush", FSourceFilterStyle::FilterSetBrush);

	StyleSet->Set("ExpandableAreaBrush", new BOX_BRUSH("PropertyView/DetailCategoryTop", FMargin(4 / 16.0f, 8.0f / 16.0f, 4 / 16.0f, 4 / 16.0f)));

	// Normal button
	FButtonStyle Button = FButtonStyle()
		.SetNormal(BOX_BRUSH("Common/Button", FVector2D(32, 32), 8.0f / 32.0f))
		.SetHovered(BOX_BRUSH("Common/Button_Hovered", FVector2D(32, 32), 8.0f / 32.0f))
		.SetPressed(BOX_BRUSH("Common/Button_Pressed", FVector2D(32, 32), 8.0f / 32.0f))
		.SetNormalPadding(FMargin(2, 2, 2, 2))
		.SetPressedPadding(FMargin(2, 3, 2, 1));
	
	struct ButtonColor
	{
	public:
		FName Name;
		FLinearColor Normal;
		FLinearColor Hovered;
		FLinearColor Pressed;

		ButtonColor(const FName& InName, const FLinearColor& Color) : Name(InName)
		{
			Normal = Color * 0.8f;
			Normal.A = Color.A;
			Hovered = Color * 1.0f;
			Hovered.A = Color.A;
			Pressed = Color * 0.6f;
			Pressed.A = Color.A;
		}
	};


	TArray< ButtonColor > FlatButtons;
	FlatButtons.Add(ButtonColor("SourceFilter.Filter", FLinearColor(0.25f, 0.25f, 0.25f, 0.9f)));

	FlatButtons.Add(ButtonColor("SourceFilter.FilterSetOperation.NOT", FLinearColor(0.85f, 0.25f, 0.25f, 0.9f)));
	ButtonColor ANDButtonColors("SourceFilter.FilterSetOperation.AND", FLinearColor(0.25f, 0.85f, 0.25f, 0.9f));
	FlatButtons.Add(ANDButtonColors);
	FlatButtons.Add(ButtonColor("SourceFilter.FilterSetOperation.OR", FLinearColor(0.25f, 0.25f, 0.85f, 0.9f)));

	FlatButtons.Add(ButtonColor("SourceFilter.FilterSet", FLinearColor(0.125f, 0.125f, 0.125f, 0.9f)));

	StyleSet->Set("WorldFilterToggleButton", FCheckBoxStyle()		
		.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
		.SetUncheckedImage(BOX_BRUSH("Common/FlatButton", 4.0f / 16.0f, FLinearColor(1, 1, 1, 0.1f)))
		.SetUncheckedHoveredImage(BOX_BRUSH("Common/FlatButton", 4.0f / 16.0f, ANDButtonColors.Hovered))  
		.SetUncheckedPressedImage(BOX_BRUSH("Common/FlatButton", 4.0f / 16.0f, ANDButtonColors.Pressed)) 
		.SetCheckedImage(BOX_BRUSH("Common/FlatButton", 4.0f / 16.0f, ANDButtonColors.Normal)) 
		.SetCheckedHoveredImage(BOX_BRUSH("Common/FlatButton", 4.0f / 16.0f, ANDButtonColors.Hovered)) 
		.SetCheckedPressedImage(BOX_BRUSH("Common/FlatButton", 4.0f / 16.0f, ANDButtonColors.Pressed)) 
	);
	
	for (const ButtonColor& Entry : FlatButtons)
	{
		StyleSet->Set(Entry.Name, FButtonStyle(Button)
			.SetNormal(BOX_BRUSH("Common/FlatButton", 2.0f / 8.0f, Entry.Normal))
			.SetHovered(BOX_BRUSH("Common/FlatButton", 2.0f / 8.0f, Entry.Hovered))
			.SetPressed(BOX_BRUSH("Common/FlatButton", 2.0f / 8.0f, Entry.Pressed))
		);
	}

	StyleSet->Set("SourceFilter.Splitter", FSplitterStyle()
		.SetHandleNormalBrush(FSlateColorBrush(FLinearColor(FColor(32, 32, 32))))
		.SetHandleHighlightBrush(FSlateColorBrush(FLinearColor(FColor(96, 96, 96))))
	);

	StyleSet->Set("ToggleButton", FButtonStyle(Button)
		.SetNormal(FSlateNoResource())
		.SetHovered(BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, FLinearColor(0.701f, 0.225f, 0.003f)))
		.SetPressed(BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, FLinearColor(0.701f, 0.225f, 0.003f)))
	);

	FComboButtonStyle ToolbarComboButton = FComboButtonStyle()
		.SetButtonStyle(StyleSet->GetWidgetStyle<FButtonStyle>("ToggleButton"))
		.SetDownArrowImage(IMAGE_BRUSH("Common/ShadowComboArrow", Icon8x8))
		.SetMenuBorderBrush(BOX_BRUSH("Old/Menu_Background", FMargin(8.0f / 64.0f)))
		.SetMenuBorderPadding(FMargin(0.0f));
	StyleSet->Set("SourceFilter.ComboButton", ToolbarComboButton);

	StyleSet->Set("SourceFilter.ToolBarIcon", new IMAGE_BRUSH_SVG("Starship/AssetIcons/Character_64", Icon32x32));
	StyleSet->Set("SourceFilter.TabIcon", new IMAGE_BRUSH_SVG("Starship/AssetIcons/Character_16", Icon16x16));

	StyleSet->Set("SourceFilter.NewPreset", new IMAGE_BRUSH("Icons/icon_file_saveas_16px", Icon16x16));
	StyleSet->Set("SourceFilter.LoadPreset", new IMAGE_BRUSH("Icons/icon_file_open_16px", Icon16x16));
	StyleSet->Set("SourceFilter.SavePreset", new IMAGE_BRUSH_SVG("Starship/Common/SaveCurrent", Icon16x16));


	StyleSet->Set("SourceFilter.TextStyle", FTextBlockStyle(NormalText)
		.SetFont(DEFAULT_FONT("Bold", 9))
		.SetColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f, 0.9f))
		.SetShadowOffset(FVector2D(1, 1))
		.SetShadowColorAndOpacity(FLinearColor(0, 0, 0, 0.9f)));

	StyleSet->Set("SourceFilter.DragDrop.Border", new BOX_BRUSH("Old/Menu_Background", FMargin(8.0f / 64.0f)));

	StyleSet->Set("FontAwesome.9", REGULAR_ICON_FONT(9));
	StyleSet->Set("FontAwesome.12", REGULAR_ICON_FONT(12));
		
	FSlateStyleRegistry::RegisterSlateStyle( *StyleSet.Get() );
}

#undef RootToContentDir

void FSourceFilterStyle::Shutdown()
{
	if( StyleSet.IsValid() )
	{
		FSlateStyleRegistry::UnRegisterSlateStyle( *StyleSet.Get() );
		ensure( StyleSet.IsUnique() );
		StyleSet.Reset();
	}
}

const ISlateStyle& FSourceFilterStyle::Get()
{
	return *( StyleSet.Get() );
}

const FName& FSourceFilterStyle::GetStyleSetName()
{
	return StyleSet->GetStyleSetName();
}
