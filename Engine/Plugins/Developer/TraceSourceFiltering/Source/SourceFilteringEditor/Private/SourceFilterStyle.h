// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class ISlateStyle;
struct FTextBlockStyle;

class FSourceFilterStyle
{
public:
	static void Initialize();
	static void Shutdown();

	static const ISlateStyle& Get();
	static const FName& GetStyleSetName();
	static const FSlateBrush* GetBrush(FName PropertyName, const ANSICHAR* Specifier = NULL)
	{
		return StyleSet->GetBrush(PropertyName, Specifier);
	}

private:
	static FSlateBrush* FilterBrush;
	static FSlateBrush* FilterSetBrush;
	/** Singleton instances of this style. */
	static TSharedPtr< class FSlateStyleSet > StyleSet;	
	static FTextBlockStyle NormalText;
};
