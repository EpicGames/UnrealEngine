// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceFilteringEditorModule.h"
#include "Framework/Commands/UICommandList.h"
#include "Modules/ModuleManager.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Insights/IUnrealInsightsModule.h"
#include "Framework/Commands/Commands.h"
#include "Widgets/Docking/SDockTab.h"
#include "Misc/ConfigContext.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "EmptySourceFilter.h"
#include "SourceFilterCollection.h"
#include "TraceSourceFiltering.h"

#include "STraceSourceFilteringWidget.h"
#include "SourceFilterStyle.h"

#include "IGameplayInsightsModule.h"

#include "Engine/Blueprint.h"
#include "Editor.h"

IMPLEMENT_MODULE(FSourceFilteringEditorModule, SourceFilteringEditor);

FString FSourceFilteringEditorModule::SourceFiltersIni;
FName InsightsSourceFilteringTabName(TEXT("InsightsSourceFiltering"));

#define LOCTEXT_NAMESPACE "FSourceFilteringEditorModule"

class FSourceFilteringCommands : public TCommands<FSourceFilteringCommands>
{
public:
	FSourceFilteringCommands()
		: TCommands<FSourceFilteringCommands>("TraceSourceFiltering", LOCTEXT("TraceSourceFiltering", "Trace Source Filtering"), NAME_None, "SourceFilter")
	{ }

	virtual void RegisterCommands() override
	{
		UI_COMMAND( ToggleSourceFilteringVisibility,
			"Source Filters",
			"Opens the Trace Source Filtering tab, allows for filtering UWorld and AActor instances to not output Trace data",
			EUserInterfaceActionType::ToggleButton, FInputChord());
	}
	
	TSharedPtr<FUICommandInfo> ToggleSourceFilteringVisibility;
};


void FSourceFilteringEditorModule::StartupModule()
{
	LLM_SCOPE_BYNAME(TEXT("Insights/SourceFilteringEditor"));
	
	FSourceFilterStyle::Initialize();
	FSourceFilteringCommands::Register();

	// Populate static ini path
	FConfigContext::ReadIntoGConfig().Load(TEXT("TraceSourceFilters"), SourceFiltersIni);
	
	IUnrealInsightsModule& UnrealInsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");	
	FOnRegisterMajorTabExtensions& TimingProfilerLayoutExtension = UnrealInsightsModule.OnRegisterMajorTabExtension(FInsightsManagerTabs::TimingProfilerTabId);
	TimingProfilerLayoutExtension.AddRaw(this, &FSourceFilteringEditorModule::RegisterLayoutExtensions);
	
#if WITH_EDITOR
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	AssetRegistryModule.Get().OnInMemoryAssetDeleted().AddRaw(this, &FSourceFilteringEditorModule::HandleAssetDeleted);
	FEditorDelegates::OnAssetsPreDelete.AddRaw(this, &FSourceFilteringEditorModule::OnAssetsPendingDelete);
#endif // WITH_EDITOR
}

void FSourceFilteringEditorModule::ShutdownModule()
{
	FSourceFilterStyle::Shutdown();
}



void FSourceFilteringEditorModule::ToggleSourceFilteringVisibility()
{
	TSharedPtr<SDockTab> SourceFilterTab = InsightsTabManager->FindExistingLiveTab(InsightsSourceFilteringTabName);
	if (SourceFilterTab.IsValid())
	{
		SourceFilterTab->RequestCloseTab();
	}
	else
	{
		InsightsTabManager->TryInvokeTab(InsightsSourceFilteringTabName);
	}
}

bool FSourceFilteringEditorModule::IsSourceFilteringVisibile()
{
	return bIsSourceFilterTabOpen;
}

void FSourceFilteringEditorModule::RegisterLayoutExtensions(FInsightsMajorTabExtender& InOutExtender)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/SourceFilteringEditor"));
#if WITH_EDITOR
	FTabId ExtendedTabId(GameplayInsightsTabs::DocumentTab);
#else
	FTabId ExtendedTabId(FTimingProfilerTabs::TimersID);
#endif

	InsightsTabManager = InOutExtender.GetTabManager();
	TSharedPtr<FUICommandList> CommandList = MakeShared<FUICommandList>();
	const FSourceFilteringCommands& Commands = FSourceFilteringCommands::Get();

	CommandList->MapAction(Commands.ToggleSourceFilteringVisibility, 
							FExecuteAction::CreateRaw(this, &FSourceFilteringEditorModule::ToggleSourceFilteringVisibility), 
							FCanExecuteAction(),
							FIsActionChecked::CreateRaw(this, &FSourceFilteringEditorModule::IsSourceFilteringVisibile));
	
	InOutExtender.GetMenuExtender()->AddToolBarExtension( "MainToolbar", EExtensionHook::First, CommandList,
		FToolBarExtensionDelegate::CreateLambda([](FToolBarBuilder& ToolBarBuilder)
		{
			ToolBarBuilder.AddToolBarButton(FSourceFilteringCommands::Get().ToggleSourceFilteringVisibility,
				NAME_None, TAttribute<FText>(), TAttribute<FText>(),
				FSlateIcon(FSourceFilterStyle::GetStyleSetName(), "SourceFilter.ToolBarIcon"));
		}));
	
	InOutExtender.GetLayoutExtender().ExtendLayout(ExtendedTabId, ELayoutExtensionPosition::Before, FTabManager::FTab(InsightsSourceFilteringTabName, ETabState::ClosedTab));

	TSharedRef<FWorkspaceItem> Category = InOutExtender.GetTabManager()->AddLocalWorkspaceMenuCategory(NSLOCTEXT("FInsightsSourceFilteringModule", "SourceFilteringGroupName", "Filtering"));

	FMinorTabConfig& MinorTabConfig = InOutExtender.AddMinorTabConfig();
	MinorTabConfig.TabId = InsightsSourceFilteringTabName;
	MinorTabConfig.TabLabel = LOCTEXT("SourceFilteringTab", "Trace Source Filtering");
	MinorTabConfig.TabTooltip = LOCTEXT("SourceFilteringTabTooltip", "Opens the Trace Source Filtering tab, allows for filtering UWorld and AActor instances to not output Trace data");
	MinorTabConfig.TabIcon = FSlateIcon(FSourceFilterStyle::GetStyleSetName(), "SourceFilter.TabIcon");
	MinorTabConfig.WorkspaceGroup = Category;
	MinorTabConfig.OnSpawnTab = FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs& Args)
	{
		bIsSourceFilterTabOpen = true;
		
		const TSharedRef<SDockTab> SourceFilterTab = SNew(SDockTab)
			.TabRole(ETabRole::PanelTab)
			.OnTabClosed_Lambda([this](TSharedRef<SDockTab>){ bIsSourceFilterTabOpen = false; });

		TSharedRef<STraceSourceFilteringWidget> Window = SNew(STraceSourceFilteringWidget);
		SourceFilterTab->SetContent(Window);

		return SourceFilterTab;
	});
}

#if WITH_EDITOR
void FSourceFilteringEditorModule::HandleAssetDeleted(UObject* DeletedObject)
{
	// Remove all pending filter deletions for which the class is actually deleted
	PendingDeletions.RemoveAll([&DeletedObject](FPendingFilterDeletion& PendingDelete)
	{
		return PendingDelete.ToDeleteFilterClassObject == DeletedObject;
	});
}

void FSourceFilteringEditorModule::OnAssetsPendingDelete(TArray<UObject*> const& ObjectsForDelete)
{
	USourceFilterCollection* FilterCollection = FTraceSourceFiltering::Get().GetFilterCollection();

	if (FilterCollection)
	{
		TArray<TObjectPtr<UDataSourceFilter>> Filters;
		FilterCollection->GetFlatFilters(Filters);
		for (UDataSourceFilter* Filter : Filters) 
		{
			// Check whether or not the to-be deleted objects contains this filter its blueprint class
			UObject* const* DeletedClassObjectPtr = ObjectsForDelete.FindByPredicate([Filter](UObject* Obj) -> bool
			{
				if (UBlueprint* Blueprint = Cast<UBlueprint>(Obj))
				{
					return Blueprint->GeneratedClass.Get() == Filter->GetClass();
				}

				return false;
			});
			
			// If so enqueue a pending deletion, used to keep track of cancelled deletions
			if (DeletedClassObjectPtr)
			{
				UEmptySourceFilter* EmptyFilter = NewObject<UEmptySourceFilter>(FilterCollection);
				EmptyFilter->MissingClassName = Filter->GetClass()->GetName();

				PendingDeletions.Add({ Filter, EmptyFilter, *DeletedClassObjectPtr });
				// Replace the to-be removed filter with an empty filter instance
				FilterCollection->ReplaceFilter(Filter, EmptyFilter);
			}
		}

		if (PendingDeletions.Num())
		{
			// In case filters are due to be removed, enqueue a callback during the next frame's tick to handle cancelled deletes
			GEditor->GetTimerManager()->SetTimerForNextTick([FilterCollection, this]()
			{
				for (FPendingFilterDeletion& PendingDeletion : PendingDeletions)
				{
					FilterCollection->ReplaceFilter(PendingDeletion.ReplacementFilter, PendingDeletion.FilterWithDeletedClass);
				}
				PendingDeletions.Empty();
			});
		}
	}


}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
