// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

#include "Templates/SharedPointer.h"

class FTabManager;

struct FInsightsMajorTabExtender;

class FSourceFilteringEditorModule : public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	static FString SourceFiltersIni;

protected:
	void RegisterLayoutExtensions(FInsightsMajorTabExtender& InOutExtender);
	
	void ToggleSourceFilteringVisibility();
	bool IsSourceFilteringVisibile();

	TSharedPtr<FTabManager> InsightsTabManager;
	bool bIsSourceFilterTabOpen = false;

#if WITH_EDITOR
	/** Keeps track of any source filters due to be removed because their class (blueprint) is due to be deleted */
	struct FPendingFilterDeletion
	{
		class UDataSourceFilter* FilterWithDeletedClass;
		class UDataSourceFilter* ReplacementFilter;
		const UObject* ToDeleteFilterClassObject;
	};
	TArray<FPendingFilterDeletion> PendingDeletions;
	
	/** Callbacks to handle blueprint source filter class deletion */
	void HandleAssetDeleted(UObject* DeletedObject);
	void OnAssetsPendingDelete(TArray<UObject*> const& ObjectsForDelete);
#endif // WITH_EDITOR
};
