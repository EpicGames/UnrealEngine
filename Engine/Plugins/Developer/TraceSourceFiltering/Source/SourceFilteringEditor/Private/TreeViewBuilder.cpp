// Copyright Epic Games, Inc. All Rights Reserved.

#include "TreeViewBuilder.h"
#include "IFilterObject.h"

void FTreeViewDataBuilder::AddFilterObject(TSharedRef<IFilterObject> FilterObject)
{	
	FilterObjects.Add(FilterObject);
	FlatFilterObjects.Add(FilterObject);
}

void FTreeViewDataBuilder::AddChildFilterObject(TArray<TSharedPtr<IFilterObject>> InFilterObjects, TSharedRef<IFilterObject> ParentObject)
{
	for (TSharedPtr<IFilterObject> ChildFilter: InFilterObjects)
	{
		FlatFilterObjects.Add(ChildFilter);
	}

	ParentToChildren.FindOrAdd(ParentObject).Append(InFilterObjects);
}
