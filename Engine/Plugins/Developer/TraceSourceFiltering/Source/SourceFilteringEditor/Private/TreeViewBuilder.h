// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class IFilterObject;

/** Helper structure allowing a ISessionSourceFilterService to populate the data use to generate the SSourceFilteringTreeView in STraceSourceFilteringWidget */
struct FTreeViewDataBuilder
{
	FTreeViewDataBuilder(TArray<TSharedPtr<IFilterObject>>& InFilterObjects, TMap<TSharedPtr<IFilterObject>, TArray<TSharedPtr<IFilterObject>>>& InParentToChildren, TArray<TSharedPtr<IFilterObject>>& InFlatFilterObjects) : FilterObjects(InFilterObjects), ParentToChildren(InParentToChildren), FlatFilterObjects(InFlatFilterObjects) {}

	/** Add a root-level filter */
	void AddFilterObject(TSharedRef<IFilterObject> FilterObject);
	/** Add a set of child-filters for a, previously added, parent filter */
	void AddChildFilterObject(TArray<TSharedPtr<IFilterObject>> FilterObjects, TSharedRef<IFilterObject> ParentObject);

protected:
	/** Root-level filter objects  */
	TArray<TSharedPtr<IFilterObject>>& FilterObjects;
	/** Parent / child filter(s) mapping */
	TMap<TSharedPtr<IFilterObject>, TArray<TSharedPtr<IFilterObject>>>& ParentToChildren;
	/** Flat list of all the filter objects contained in both FilterObjects and ParentToChildren */
	TArray<TSharedPtr<IFilterObject>>& FlatFilterObjects;
};
