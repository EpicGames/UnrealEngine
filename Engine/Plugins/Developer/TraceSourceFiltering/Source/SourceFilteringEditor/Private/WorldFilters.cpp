// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldFilters.h"
#include "Misc/ConfigCacheIni.h"
#include "Widgets/Input/SCheckBox.h" 
#include "Widgets/SBoxPanel.h"

#include "SourceFilteringEditorModule.h"
#include "SourceFilterStyle.h"
#include "Widgets/Text/STextBlock.h"

#if WITH_ENGINE
#include "Engine/EngineBaseTypes.h"
#include "Engine/EngineTypes.h"
#endif // WITH_ENGINE

#define LOCTEXT_NAMESPACE "FWorldTypeTraceFilter"

FWorldTypeTraceFilter::FWorldTypeTraceFilter(const TFunction<void(uint8, bool)>& InOnSetWorldTypeFilterState, const TFunction<bool(uint8)>& InOnGetWorldTypeFilterState) : OnSetWorldTypeFilterState(InOnSetWorldTypeFilterState), OnGetWorldTypeFilterState(InOnGetWorldTypeFilterState)
{
	WorldTypeFilterName = LOCTEXT("WorldTypeFilterName", "By World Type");
	WorldTypeFilterValues.Add(TPair<FText, TArray<uint8>>(LOCTEXT("EditorWorldFilterName", "Editor"), TArray<uint8>({ 2 /*EWorldType::Editor*/, 4 /*EWorldType::EditorPreview*/ })));
	WorldTypeFilterValues.Add(TPair<FText, TArray<uint8>>(LOCTEXT("PIEWorldFilterName", "PIE"), TArray<uint8>({ 3 /*EWorldType::PIE*/ })));
	WorldTypeFilterValues.Add(TPair<FText, TArray<uint8>>(LOCTEXT("GameWorldFilterName", "Game"), TArray<uint8>({ 1 /*EWorldType::Game*/, 5 /*EWorldType::GamePreview*/, 6 /*EWorldType::GameRPC*/ })));

#if WITH_ENGINE
	static_assert((uint8)EWorldType::Editor == 2, "EWorldType and WorldTypeFilterValues must be kept in sync");
	static_assert((uint8)EWorldType::EditorPreview == 4, "EWorldType and WorldTypeFilterValues must be kept in sync");
	static_assert((uint8)EWorldType::PIE == 3, "EWorldType and WorldTypeFilterValues must be kept in sync");
	static_assert((uint8)EWorldType::Game == 1, "EWorldType and WorldTypeFilterValues must be kept in sync");
	static_assert((uint8)EWorldType::GamePreview == 5, "EWorldType and WorldTypeFilterValues must be kept in sync");
	static_assert((uint8)EWorldType::GameRPC == 6, "EWorldType and WorldTypeFilterValues must be kept in sync");
#endif

	LoadSettings();
}

FText FWorldTypeTraceFilter::GetDisplayText()
{
	return LOCTEXT("WorldTypeTraceFilterLabel", "Filter by World Type");
}

FText FWorldTypeTraceFilter::GetToolTipText()
{
	return LOCTEXT("WorldTypeTraceFilterToolTip", "Marks World's on the analyis session as Traceable if it is of (any) specified World Type(s)");
}

TSharedRef<SWidget> FWorldTypeTraceFilter::GenerateWidget()
{
	TSharedPtr<SHorizontalBox> ToggleButtonBox = SNew(SHorizontalBox);
	
	for (const TPair<FText, TArray<uint8>>& TypePair : WorldTypeFilterValues)
	{
		ToggleButtonBox->AddSlot()
		.AutoWidth()
		.Padding(0, 0, 2.f, 0.f)
		[
			SNew(SCheckBox)
			.Style(FSourceFilterStyle::Get(), "WorldFilterToggleButton")
			.IsChecked_Lambda([this, TypePair]() -> ECheckBoxState
			{
				return OnGetWorldTypeFilterState(TypePair.Value[0]) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			})
			.Padding(4.f)
			.OnCheckStateChanged_Lambda([this, TypePair](ECheckBoxState State)
			{
				const bool bNewState = State == ECheckBoxState::Checked;
				for (const uint8& Type : TypePair.Value)
				{
					OnSetWorldTypeFilterState(Type, bNewState);
				}

				SaveSettings();
			})
			[
				SNew(STextBlock)
				.TextStyle(FSourceFilterStyle::Get(), "SourceFilter.TextStyle")
				.Text(TypePair.Key)
			]
		];
	}

	return ToggleButtonBox.ToSharedRef();
}

void FWorldTypeTraceFilter::SaveSettings()
{
	FString IniSectionName = TEXT("WorldTypeTraceFilter");
	FString IniDisabledTypesKey = TEXT("DisabledWorldTypes");

	FString Separator = TEXT("+");
	FString DisabledWorldTypeString;
	for (const TPair<FText, TArray<uint8>>& WorldTypePair : WorldTypeFilterValues)
	{
		for (uint8 WorldTypeValue : WorldTypePair.Value)
		{
			const bool bDisabled = !OnGetWorldTypeFilterState(WorldTypeValue);
			if (bDisabled)
			{
				DisabledWorldTypeString += FString::FromInt(WorldTypeValue);
				DisabledWorldTypeString += Separator;
			}
		}
	}
	DisabledWorldTypeString.RemoveFromEnd(Separator);	

	GConfig->SetString(*IniSectionName, *IniDisabledTypesKey, *DisabledWorldTypeString, FSourceFilteringEditorModule::SourceFiltersIni);
}

void FWorldTypeTraceFilter::LoadSettings()
{
	FString IniSectionName = TEXT("WorldTypeTraceFilter");
	FString IniDisabledTypesKey = TEXT("DisabledWorldTypes");

	FString DisabledTypesString;
	GConfig->GetString(*IniSectionName, *IniDisabledTypesKey, DisabledTypesString, FSourceFilteringEditorModule::SourceFiltersIni);

	FString Separator = TEXT("+");
	TArray<FString> DisabledWorldTypeStrings;
	DisabledTypesString.ParseIntoArray(DisabledWorldTypeStrings, *Separator);
	
	TArray<uint8> DisabledWorldTypes;
	Algo::Transform(DisabledWorldTypeStrings, DisabledWorldTypes, [](FString Value)
	{
		return FCString::Atoi(*Value);
	});

	for (const TPair<FText, TArray<uint8>>& WorldTypePair : WorldTypeFilterValues)
	{
		for (uint8 WorldTypeValue : WorldTypePair.Value)
		{
			const bool bState = OnGetWorldTypeFilterState(WorldTypeValue);
			const bool bIniState = !DisabledWorldTypes.Contains(WorldTypeValue);

			if (bState != bIniState)
			{
				OnSetWorldTypeFilterState(WorldTypeValue, bIniState);
			}
		}
	}
}

FWorldNetModeTraceFilter::FWorldNetModeTraceFilter(const TFunction<void(uint8, bool)>& InOnSetWorldNetModeFilterState, const TFunction<bool(uint8)>& InOnGetWorldNetModeFilterState) : OnSetWorldNetModeFilterState(InOnSetWorldNetModeFilterState), OnGetWorldNetModeFilterState(InOnGetWorldNetModeFilterState)
{
	WorldNetModeFilterName = LOCTEXT("ByNetModeWorldFilterName", "By Net Mode");

	WorldNetModeFilterValues.Add(TPair<FText, TArray<uint8>>(LOCTEXT("ClientNetModeWorldFilterName", "Client"), TArray<uint8>({ 3 /*NM_Client*/ })));
	WorldNetModeFilterValues.Add(TPair<FText, TArray<uint8>>(LOCTEXT("ServerNetModeWorldFilterName", "Server"), TArray<uint8>({ 1 /*NM_DedicatedServer*/, 2 /*NM_ListenServer*/ })));
	WorldNetModeFilterValues.Add(TPair<FText, TArray<uint8>>(LOCTEXT("StandaloneNetModeWorldFilterName", "Standalone"), TArray<uint8>({ 0 /*NM_Standalone*/ })));
	
#if WITH_ENGINE
	static_assert((uint8)ENetMode::NM_Client == 3, "ENetMode and WorldNetModeFilterValues must be kept in sync");
	static_assert((uint8)ENetMode::NM_DedicatedServer == 1, "ENetMode and WorldNetModeFilterValues must be kept in sync");
	static_assert((uint8)ENetMode::NM_ListenServer == 2, "ENetMode and WorldNetModeFilterValues must be kept in sync");
	static_assert((uint8)ENetMode::NM_Standalone == 0, "ENetMode and WorldNetModeFilterValues must be kept in sync");
#endif

	LoadSettings();
}

FText FWorldNetModeTraceFilter::GetDisplayText()
{
	return LOCTEXT("WorldNetModeTraceFilterLabel", "Filter by World Net Mode");
}

FText FWorldNetModeTraceFilter::GetToolTipText()
{
	return LOCTEXT("WorldNetModeTraceFilterToolTip", "Marks World's on the analysis session as Traceable if it is running in (any) of the specified Net Mode(s)");
}

TSharedRef<SWidget> FWorldNetModeTraceFilter::GenerateWidget()
{
	TSharedPtr<SHorizontalBox> ToggleButtonBox = SNew(SHorizontalBox);

	for (const TPair<FText, TArray<uint8>>& TypePair : WorldNetModeFilterValues)
	{
		ToggleButtonBox->AddSlot()
		.AutoWidth()
		.Padding(0, 0, 2.f, 0.f)
		[
			SNew(SCheckBox)
			.Style(FSourceFilterStyle::Get(), "WorldFilterToggleButton")
			.IsChecked_Lambda([this, TypePair]() -> ECheckBoxState
			{
				return OnGetWorldNetModeFilterState(TypePair.Value[0]) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			})
			.Padding(4.f)
			.OnCheckStateChanged_Lambda([this, TypePair](ECheckBoxState State)
			{
				const bool bNewState = State == ECheckBoxState::Checked;
				for (const uint8& Type : TypePair.Value)
				{
					OnSetWorldNetModeFilterState(Type, bNewState);
				}

				SaveSettings();
			})
			[
				SNew(STextBlock)
				.TextStyle(FSourceFilterStyle::Get(), "SourceFilter.TextStyle")
				.Text(TypePair.Key)
			]
		];
	}

	return ToggleButtonBox.ToSharedRef();
}

void FWorldNetModeTraceFilter::SaveSettings()
{
	FString IniSectionName = TEXT("WorldNetModeTraceFilter");
	FString IniDisabledTypesKey = TEXT("DisabledNetModeTypes");

	FString Separator = TEXT("+");
	FString DisabledWorldNetModeString;
	for (const TPair<FText, TArray<uint8>>& WorldNetModePair : WorldNetModeFilterValues)
	{
		for (uint8 WorldNetModeValue : WorldNetModePair.Value)
		{
			const bool bDisabled = !OnGetWorldNetModeFilterState(WorldNetModeValue);
			if (bDisabled)
			{
				DisabledWorldNetModeString += FString::FromInt(WorldNetModeValue);
				DisabledWorldNetModeString += Separator;
			}
		}
	}
	DisabledWorldNetModeString.RemoveFromEnd(Separator);

	GConfig->SetString(*IniSectionName, *IniDisabledTypesKey, *DisabledWorldNetModeString, FSourceFilteringEditorModule::SourceFiltersIni);
}

void FWorldNetModeTraceFilter::LoadSettings()
{
	FString IniSectionName = TEXT("WorldNetModeTraceFilter");
	FString IniDisabledTypesKey = TEXT("DisabledNetModeTypes");

	FString DisabledTypesString;
	GConfig->GetString(*IniSectionName, *IniDisabledTypesKey, DisabledTypesString, FSourceFilteringEditorModule::SourceFiltersIni);

	FString Separator = TEXT("+");
	TArray<FString> DisabledWorldNetModeStrings;
	DisabledTypesString.ParseIntoArray(DisabledWorldNetModeStrings, *Separator);

	TArray<uint8> DisabledWorldNetModes;
	Algo::Transform(DisabledWorldNetModeStrings, DisabledWorldNetModes, [](FString Value)
	{
		return FCString::Atoi(*Value);
	});

	for (const TPair<FText, TArray<uint8>>& WorldNetModePair : WorldNetModeFilterValues)
	{
		for (uint8 WorldNetModeValue : WorldNetModePair.Value)
		{
			const bool bState = OnGetWorldNetModeFilterState(WorldNetModeValue);
			const bool bIniState = !DisabledWorldNetModes.Contains(WorldNetModeValue);

			if (bState != bIniState)
			{
				OnSetWorldNetModeFilterState(WorldNetModeValue, bIniState);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE // "FWorldTypeTraceFilter"
