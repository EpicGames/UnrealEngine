// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "ISessionSourceFilterService.h"
#include "Internationalization/Text.h"

/** World filter implementation, that marks Worlds as traceable according to their EWorldType */
class FWorldTypeTraceFilter : public IWorldTraceFilter
{
public:
	FWorldTypeTraceFilter(const TFunction<void(uint8, bool)>& InOnSetWorldTypeFilterState, const TFunction<bool(uint8)>& InOnGetWorldTypeFilterState);

	/** Begin IWorldTraceFilter Overrides */
	virtual FText GetDisplayText() override;
	virtual FText GetToolTipText() override;
	virtual TSharedRef<SWidget> GenerateWidget() override;
	/** End IWorldTraceFilter Overrides */
protected:	
	/** Load / save any settings to config that should be persistent between editor sessions */
	void SaveSettings();	
	void LoadSettings();

protected:
	TFunction<void(uint8, bool)> OnSetWorldTypeFilterState;
	TFunction<bool(uint8)> OnGetWorldTypeFilterState;

	FText WorldTypeFilterName;
	TArray<TPair<FText, TArray<uint8>>> WorldTypeFilterValues;
};

/** World filter implementation, that marks Worlds as traceable according to their ENetMode */
class FWorldNetModeTraceFilter : public IWorldTraceFilter
{
public:
	FWorldNetModeTraceFilter(const TFunction<void(uint8, bool)>& InOnSetWorldNetModeFilterState, const TFunction<bool(uint8)>& InOnGetWorldNetModeFilterState);
	
	/** Begin IWorldTraceFilter Overrides */
	virtual FText GetDisplayText() override;
	virtual FText GetToolTipText() override;
	virtual TSharedRef<SWidget> GenerateWidget() override;
	/** End IWorldTraceFilter Overrides */
protected:
	/** Load / save any settings to config that should be persistent between editor sessions */
	void SaveSettings();
	void LoadSettings();
protected:
	TFunction<void(uint8, bool)> OnSetWorldNetModeFilterState;
	TFunction<bool(uint8)> OnGetWorldNetModeFilterState;

	FText WorldNetModeFilterName;
	TArray<TPair<FText, TArray<uint8>>> WorldNetModeFilterValues;
};
