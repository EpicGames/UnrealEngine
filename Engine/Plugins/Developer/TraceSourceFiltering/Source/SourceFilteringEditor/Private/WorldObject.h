// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"

/** Object representing an UWorld and its filtering state */
class FWorldObject : public TSharedFromThis<FWorldObject>
{
public:
	FWorldObject(const FString& InName, uint8 InWorldType, bool bState, uint32 InHash) : Name(InName), WorldType(InWorldType), bCanOutputData(bState), Hash(InHash) {}

	const FText GetDisplayText() const { return FText::FromString(Name); }
	const FString& GetName() const { return Name; }
	bool CanOutputData() const { return bCanOutputData; }
	const uint32 GetHash() const { return Hash; }
protected:
	FString Name;
	// Corresponds to EWorldType in Engine
	uint8 WorldType;
	bool bCanOutputData;
	uint32 Hash;
};
