// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SourceFilteringEditor : ModuleRules
{
    public SourceFilteringEditor(ReadOnlyTargetRules Target) : base(Target)
    {
        PrivateDependencyModuleNames.AddRange(
                new string[] {
                    "Core",
                    "CoreUObject",
                    "Slate",
                    "SlateCore",
                    "InputCore",
                    "TraceServices",
                    "TraceInsights",
					"TraceAnalysis",
					"WorkspaceMenuStructure",
                    "SourceFilteringCore",
					"PropertyPath",
					"GameplayInsights"
                }
            );
		
        if (Target.bCompileAgainstEngine)
        {
            PrivateDependencyModuleNames.AddRange(
				new string[] {
					 "Engine",
					 "SourceFilteringTrace"
				}
			);
        }

        if (Target.bBuildEditor)
        {
            PrivateDependencyModuleNames.AddRange(
				new string[] {
					"EditorFramework",
					"UnrealEd",                 
				}
			);
        }
    }
}
