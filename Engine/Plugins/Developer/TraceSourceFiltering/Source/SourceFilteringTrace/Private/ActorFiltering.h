// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataSourceFiltering.h"
#include "Engine/Level.h"
#include "Engine/World.h"

/** Simplified version of TActorIterator */
struct FFilteredActorIterator
{
	friend class FFilteredActorCollector;

public:
	FORCEINLINE const AActor* operator*() const
	{
		return GetActorChecked();
	}

	FORCEINLINE uint32 Index() const
	{
		return CurrentIndex;
	}

	FORCEINLINE const AActor* operator->() const
	{
		return GetActorChecked();
	}

	FORCEINLINE const AActor* GetActorChecked() const
	{
		const AActor* Actor = CurrentActor.Get();
		check(Actor);
		checkf(!Actor->IsUnreachable(), TEXT("%s"), *CurrentActor->GetFullName());
		return Actor;
	}

	FORCEINLINE explicit operator bool() const
	{
		return CurrentActor != nullptr && CurrentIndex < EndIndex;
	}

	void operator++()
	{
		CurrentActor = nullptr;
		while (++CurrentIndex < EndIndex)
		{
			CurrentActor = Actors[CurrentIndex];
			if (const AActor* Actor =  CurrentActor.Get(); IsValid(Actor))
			{
				break;
			}
		}
	}
private:
	FFilteredActorIterator(const TArray<TWeakObjectPtr<const AActor>>& InActors, uint32 InStartIndex, uint32 InLength)
		: StartIndex(InStartIndex), CurrentIndex(InStartIndex), CurrentActor(nullptr), Actors(InActors)
	{
		if (InLength == 0)
		{
			EndIndex = Actors.Num();
		}
		else
		{
			EndIndex = FMath::Min(StartIndex + InLength, (uint32)Actors.Num());
		}

		CurrentActor = nullptr;
		if (InActors.Num())
		{
			check(InActors.IsValidIndex(StartIndex));
			check(InActors.IsValidIndex(EndIndex - 1));
			CurrentActor = Actors[StartIndex];
			if (const AActor* Actor = CurrentActor.Get(); !IsValid(Actor))
			{
				++(*this); // Advance til we hit the first valid actor. 
			}
		}
	}
		
	/** Range indices */
	uint32 StartIndex;
	uint32 EndIndex;
	/** Current index and corresponding AActor instance */
	uint32 CurrentIndex;
	TWeakObjectPtr<const AActor> CurrentActor;

	/** All Actors retrieved by FActorCollector, over which we iterate within the defined range */
	const TArray<TWeakObjectPtr<const AActor>>& Actors;
};

/** Used for retrieving AActor instances of particular classes from the owned UWorld, derived from FActorIteratorState */
class FFilteredActorCollector
{
public:
	FFilteredActorCollector(UWorld* InWorld, const TArray<FActorClassFilter>& FilteredClasses)
		: CurrentWorld(InWorld)
	{
		check(InWorld);
		check(IsInGameThread());

		UpdateFilterClasses(FilteredClasses);

		CollectActors();
	}

	/** Update the UClass instances used for filtering AActor instances */
	void UpdateFilterClasses(const TArray<FActorClassFilter>& FilteredClasses)
	{
		AllFilteredClasses.Reset();

		if (FilteredClasses.Num())
		{
			for (const FActorClassFilter& FilteredClass : FilteredClasses)
			{
				AllFilteredClasses.Add(FilteredClass.ActorClass.TryLoadClass<AActor>());
				if (FilteredClass.bIncludeDerivedClasses)
				{
					TArray<UClass*> DerivedClasses;
					GetDerivedClasses(FilteredClass.ActorClass.TryLoadClass<AActor>(), DerivedClasses, true);

					AllFilteredClasses.Append(DerivedClasses);
				}
			}
		}
		else
		{
			// In case there are not actor classes provided we use AActor and any derived class instead (heavy so non-editor only)
			AllFilteredClasses.Add(AActor::StaticClass());
#if !WITH_EDITOR
			TArray<UClass*> DerivedClasses;
			GetDerivedClasses(AActor::StaticClass(), DerivedClasses, true);
			AllFilteredClasses.Append(DerivedClasses);
#endif
		}

	}

	/** Collect actor instances within UWorld who's class is contained by AllFilteredClasses */
	void CollectActors()
	{
		ActorArray.Reset();

#if WITH_EDITOR
		// Same behaviour as FActorIteratorState to reduce editor load
		if (AllFilteredClasses.Num() == 1 && AllFilteredClasses[0].Get() == AActor::StaticClass())
		{
			// First determine the number of actors in the world to reduce reallocations when we append them to the array below.
			int32 TotalNumActors = 0;
			for (ULevel* Level : CurrentWorld->GetLevels())
			{
				if (Level)
				{
					TotalNumActors += Level->Actors.Num();
				}
			}

			// Presize the array
			ActorArray.Reserve(TotalNumActors);

			// Fill the array
			for (ULevel* Level : CurrentWorld->GetLevels())
			{
				if (Level)
				{
					ActorArray.Append(Level->Actors);
				}
			}

			return;
		}
#endif 

		/** Ignore CDOs and objects marked as pending kill */
		EObjectFlags ExcludeFlags = RF_ClassDefaultObject;
		EInternalObjectFlags InternalExcludeFlags = EInternalObjectFlags::Garbage;

		ForEachObjectOfClasses(GetAllFilteredClasses(), [this](UObject* Object)
		{
			// Ensure the actor, level and world are valid
			const AActor* Actor = static_cast<const AActor*>(Object);
			const ULevel* ActorLevel = Actor ? Actor->GetLevel() : nullptr;
			const UWorld* ActorWorld = ActorLevel ? ActorLevel->GetWorld() : nullptr;

			if (ActorWorld == CurrentWorld)
			{
				ActorArray.Add(Actor);
			}
		}, ExcludeFlags, InternalExcludeFlags);

		// Safety check to ensure we do not gather more actors than UObjects exist within the process
		check(ActorArray.Num() <= GUObjectArray.GetObjectArrayNum());
	}

	/** Retrieve iterator over the provided range */
	FFilteredActorIterator GetIterator(uint32 StartIndex = 0, uint32 Length = 0)
	{
		return FFilteredActorIterator(ActorArray, StartIndex, Length);
	}

	/** Return total number of actors 'collected' */
	int32 NumActors() const
	{
		return ActorArray.Num();
	}

	TArray<const UClass*> GetAllFilteredClasses() const
	{
		TArray<const UClass*> RawFilteredClasses;
		RawFilteredClasses.Reserve(AllFilteredClasses.Num());
		for(const TWeakObjectPtr<const UClass>& FilteredClass : AllFilteredClasses)
		{
			if (const UClass* RawFilteredClass = FilteredClass.Get())
			{
				RawFilteredClasses.Add(RawFilteredClass);
			}
		}
		return RawFilteredClasses;
	}

protected:
	UWorld* CurrentWorld;

	TArray<TWeakObjectPtr<const AActor>> ActorArray;
	TArray<TWeakObjectPtr<const UClass>> AllFilteredClasses;
};
