// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataSourceFilterSet.h"
#include "DataSourceFiltering.h"
#include "SourceFilterTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataSourceFilterSet)

void UDataSourceFilterSet::SetFilterMode(EFilterSetMode InMode)
{
	TRACE_FILTER_OPERATION(this, ESourceActorFilterOperation::SetFilterMode, (uint64)InMode);
	Mode = InMode;
}

bool UDataSourceFilterSet::DoesActorPassFilter_Internal(const AActor* InActor) const
{
	bool bPassesFilter = [this]()
	{
		if (Filters.Num() == 0)
		{
			return true;
		}

		switch (Mode)
		{
		case EFilterSetMode::OR: 
			return false;
		case EFilterSetMode::AND: 
			return true;
		case EFilterSetMode::NOT: 
			return true;
		default: 
			return false;
		}
	}();		
		
	for (const UDataSourceFilter* Filter : Filters)
	{
		if (Filter->IsEnabled())
		{
			const bool bResult = Filter->DoesActorPassFilter(InActor);
			
			// OR, at least one filter has to be passed 
			if (Mode == EFilterSetMode::OR)
			{
				bPassesFilter |= bResult;
				if (bPassesFilter)
				{
					break;
				}
			}
			// AND, all filters have to be passed 
			else if (Mode == EFilterSetMode::AND)
			{
				bPassesFilter &= bResult;

				if (!bPassesFilter)
				{
					break;
				}
			}
			// NOT, all filters have to be !passed
			else if (Mode == EFilterSetMode::NOT)
			{
				bPassesFilter &= !bResult;

				if (!bPassesFilter)
				{
					break;
				}
			}
		}
	}

	return bPassesFilter;
}

void UDataSourceFilterSet::GetDisplayText_Internal(FText& OutDisplayText) const
{
	TArray<FString> PerFilterTextStrings;
	for (const UDataSourceFilter* Filter : Filters)
	{
		FText FilterText;
		Execute_GetDisplayText(Filter, FilterText);

		PerFilterTextStrings.Add(FilterText.ToString());
	}

	const FString Prefix = [this]()
	{
		if (Mode == EFilterSetMode::NOT)
		{
			return TEXT("!");
		}
			
		return TEXT("");
	}();

	const FString PostFix = [this]()
	{
		if (Mode == EFilterSetMode::OR)
		{
			return TEXT(" || ");
		}
		else if (Mode == EFilterSetMode::AND)
		{
			return TEXT(" && ");
		}
		else if (Mode == EFilterSetMode::NOT)
		{
			return TEXT(" || ");
		}

		return TEXT(" ");
	}();
	
	FString ComposedString = TEXT("(");
	for (const FString& FilterString : PerFilterTextStrings)
	{
		ComposedString += Prefix;
		ComposedString += FilterString;
		ComposedString += PostFix;
	}

	ComposedString.RemoveFromEnd(PostFix);
	ComposedString += TEXT(")");

	OutDisplayText = FText::FromString(ComposedString);
}

EFilterSetMode UDataSourceFilterSet::GetFilterSetMode() const
{
	return Mode;
}

void UDataSourceFilterSet::SetEnabled(bool bState)
{
	Super::SetEnabled(bState);

	for (UDataSourceFilter* ChildFilter : Filters)
	{
		ChildFilter->SetEnabled(bState);
	}
}


