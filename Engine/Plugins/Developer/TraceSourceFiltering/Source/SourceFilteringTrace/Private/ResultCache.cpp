// Copyright Epic Games, Inc. All Rights Reserved.

#include "ResultCache.h"
#include "DataSourceFiltering.h"
#include "SourceFilterSet.h"

void FResultCache::Reset()
{
	NumActors = 0;

	IntervalFilterResults.Empty();
	SpawnFilterResults.Empty();

#if USE_BITSET_STORAGE
	BitResults.Empty();
#else
	Results.Empty();
#endif

	IntervalFilterIndexAndHash.Empty();
}

void FResultCache::ResetFrameData(uint32 InNumActors, uint32 InNumFilterEntries)
{
	NumActors = InNumActors;

#if USE_BITSET_STORAGE
	/** Round to nearest multiple of 32, this to ensure separate threads never try to write to the same uint32. (resulting in undefined behaviour)
	*/
	const int32 Base = NumActors / 32;
	const int32 Remainder = NumActors % 32;
	const int32 Total = (Base + !(Remainder == 0)) * 32;
	NumActorBits = Total;
	BitResults.Init(false, NumActorBits * InNumFilterEntries);
#else
	Results.SetNumZeroed(NumActors * InNumFilterEntries, false);
#endif

	IntervalFilterIndexAndHash.Reset();
	ProcessSpawnFilterResults();
}

void FResultCache::AddTickedIntervalFilter(const FFilter& Filter)
{
	IntervalFilterIndexAndHash.Add(Filter.ResultOffset, Filter.FilterHash);
}

void FResultCache::ProcessIntervalFilterResults(const TArray<uint32>& ActorHashes)
{
	if (ActorHashes.Num())
	{
		// Iterate over each interval filter that ticked this frame
		for (const TPair<uint32, uint32>& IntervalFilterPair : IntervalFilterIndexAndHash)
		{
			const uint32 FilterIndex = IntervalFilterPair.Key;
			const uint32 FilterHash = IntervalFilterPair.Value;

#if USE_BITSET_STORAGE
			const uint32 Offset = FilterIndex * NumActorBits;
			TBitArray<FDefaultBitArrayAllocator>::FConstIterator It(BitResults, Offset);
#else
			const uint32 Offset = FilterIndex * NumActors;
#endif		
			for (uint32 ActorIndex = 0; ActorIndex < NumActors; ++ActorIndex)
			{
#if USE_BITSET_STORAGE
				const bool bResult = It.GetValue();
#else
				const bool bResult = Results[Offset + ActorIndex];
#endif
				/** Add combined actor and filter hash in case it passed, remove (if contained) otherwise */
				if (bResult)
				{
					IntervalFilterResults.Add(HashCombine(FilterHash, ActorHashes[ActorIndex]));
				}
				else
				{
					IntervalFilterResults.Remove(HashCombine(FilterHash, ActorHashes[ActorIndex]));
				}

#if USE_BITSET_STORAGE
				++It;
#endif
			}
		}
	}
}

bool FResultCache::RetrieveFilterSetResult(const FFilterSet& FilterSet, const FActorFilterInfo& ActorInfo) const
{
#if USE_BITSET_STORAGE
	return BitResults[(FilterSet.ResultOffset * NumActorBits) + ActorInfo.ActorIndex];
#else
	return Results[(FilterSet.ResultOffset * NumActors) + ActorInfo.ActorIndex];
#endif
}

void FResultCache::CacheFilterResult(const FFilter& Filter, const FActorFilterInfo& ActorInfo, bool bResult)
{
	ensure(!Filter.bOnSpawnOnly);

	if (bResult)
	{
#if USE_BITSET_STORAGE
		BitResults[(Filter.ResultOffset * NumActorBits) + ActorInfo.ActorIndex] = bResult;
#else
		Results[(Filter.ResultOffset * NumActors) + ActorInfo.ActorIndex] = !!bResult;
#endif
	}
}

void FResultCache::CacheSpawnFilterResult(uint32 FilterHash, const uint32 ActorHash, bool bResult)
{
	TSet<uint32>& ResultSet = bResult ? NewSpawnFilterPassResults : NewSpawnFilterRejectResults;
	ResultSet.Add(HashCombine(FilterHash, ActorHash));
}

bool FResultCache::RetrieveCachedResult(const FFilter& Filter, const FActorFilterInfo& ActorInfo) const
{
	if (!Filter.bOnSpawnOnly && Filter.TickInterval == 1)
	{
#if USE_BITSET_STORAGE
		return BitResults[(Filter.ResultOffset * NumActorBits) + ActorInfo.ActorIndex];
#else
		return Results[(Filter.ResultOffset * NumActors) + ActorInfo.ActorIndex];
#endif 	
	}
	else if (Filter.bOnSpawnOnly)
	{
		return SpawnFilterResults.Contains(HashCombine(Filter.FilterHash, ActorInfo.ActorHash));
	}
	else if (Filter.TickInterval > 1)
	{
		return IntervalFilterResults.Contains(HashCombine(Filter.FilterHash, ActorInfo.ActorHash));
	}

	ensure(true);
	return false;
}

void FResultCache::ProcessSpawnFilterResults()
{
	if (NewSpawnFilterRejectResults.Num() || NewSpawnFilterPassResults.Num())
	{
		SpawnFilterResults.Append(NewSpawnFilterPassResults);
		for (const uint32& Hash : NewSpawnFilterRejectResults)
		{
			SpawnFilterResults.Remove(Hash);
		}
		
		NewSpawnFilterRejectResults.Empty();
		NewSpawnFilterPassResults.Empty();
	}
}

template<>
void FResultCache::CacheFilterSetResult<EFilterType::GameThreadFilters>(const FFilterSet& FilterSet, const FActorFilterInfo& ActorInfo, bool bResult)
{
	// If either we just ran the game thread filters, or the async filters (while there are no GT filters)
	if (bResult)
	{
#if USE_BITSET_STORAGE
		BitResults[(FilterSet.ResultOffset * NumActorBits) + ActorInfo.ActorIndex] = bResult;
#else
		Results[(FilterSet.ResultOffset * NumActors) + ActorInfo.ActorIndex] = bResult;
#endif
	}
}

template<>
void FResultCache::CacheFilterSetResult<EFilterType::AsynchronousFilters>(const FFilterSet& FilterSet, const FActorFilterInfo& ActorInfo, bool bResult)
{
	/** In case this filter-set contains any GT filters as well, we have to combine the two filtering results */
	if (FilterSet.bContainsGameThreadFilter)
	{
#if USE_BITSET_STORAGE
		// Non-game thread processing (happens after game thread async-task)
		FBitReference Bit = BitResults[(FilterSet.ResultOffset * NumActorBits) + ActorInfo.ActorIndex];
		const bool bGameThreadResult = Bit;

		// Combine the results according to AND or OR/NOT operation
		const bool bActualResult = FilterSet.Mode == EFilterSetMode::AND ? bGameThreadResult && bResult : bGameThreadResult || bResult;
		if (bGameThreadResult != bActualResult)
		{
			Bit = bActualResult;
		}
#else
		const bool bGameThreadResult = Results[(FilterSet.ResultOffset * NumActors) + ActorInfo.ActorIndex];
		const bool bActualResult = FilterSet.Mode == EFilterSetMode::AND ? bGameThreadResult && bResult : bGameThreadResult || bResult;
		if (bGameThreadResult != bActualResult)
		{
			Results[(FilterSet.ResultOffset * NumActors) + ActorInfo.ActorIndex] = bActualResult;
		}
#endif
	}
	else
	{
		// Can cache directly since there are no GT filter results to combine with
		CacheFilterSetResult<EFilterType::GameThreadFilters>(FilterSet, ActorInfo, bResult);
	}
}
