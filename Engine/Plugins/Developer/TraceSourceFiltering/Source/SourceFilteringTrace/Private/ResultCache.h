// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"

class AActor;
struct FFilter;
struct FFilterSet;

/** Whether or not to use a BitArray to store per-actor/filter results */
#define USE_BITSET_STORAGE 1

typedef TSet<uint32/*, DefaultKeyFuncs<uint32>, TInlineAllocator<4>*/> FHashSet;

struct FActorFilterInfo
{
	/** Actor itself */
	const AActor* Actor;
	/** Actor's GetTypeHash value */
	uint32 ActorHash;
	/** Set of filterset hashes which can be regarded as passing for this actor */
	const FHashSet* PassSets;
	/** Set of filterset hashes which can be regarded as rejected for this actor */
	const FHashSet* DiscardSets;

	/** Actor index (relative to FFilteredActorCollector::ActorArray) */
	uint32 ActorIndex;
};

enum EFilterType
{
	GameThreadFilters,
	AsynchronousFilters
};

/** Structure used to store (inter-)frame per-actor and per-filter results */
struct FResultCache
{
	/** Resets all internal data */
	void Reset();
	
	/** Resets any frame-transient data */
	void ResetFrameData(uint32 InNumActors, uint32 InNumFilterEntries);

	/** Add an entry for interval-ticked filter */
	void AddTickedIntervalFilter(const FFilter& Filter);

	/** Processes all interval-ticked filter results from this frame */
	void ProcessIntervalFilterResults(const TArray<uint32>& ActorHashes);

	/** Caches the result of evaluating a filterset for the provided actor */
	template<EFilterType FilterType>
	void CacheFilterSetResult(const FFilterSet& FilterSet, const FActorFilterInfo& ActorInfo, bool bResult);	

	/** Retrieves the, previously cached, result for a filter-set / actor combination */
	bool RetrieveFilterSetResult(const FFilterSet& FilterSet, const FActorFilterInfo& ActorInfo) const;

	/** Caches the result of evaluating a filter for the provided actor */
	void CacheFilterResult(const FFilter& Filter, const FActorFilterInfo& ActorInfo, bool bResult);

	/** Caches the result of evaluating a spawn-only filter for the provided actor */
	void CacheSpawnFilterResult(uint32 FilterHash, const uint32 ActorHash, bool bResult);
	
	/** Retrieves the, previously cached, result for a filter / actor combination */
	bool RetrieveCachedResult(const FFilter& Filter, const FActorFilterInfo& ActorInfo) const;

	/** Appends this frames spawn filter results to SpawnFilterResults */
	void ProcessSpawnFilterResults();	

protected:
	/** Cached value of total number of actors for which results could be cached */
	uint32 NumActors;

#if USE_BITSET_STORAGE
	/** Total number of actor bits to allocate */
	uint32 NumActorBits;

	TBitArray<FDefaultBitArrayAllocator> BitResults;	
#else
	TArray<bool> Results;
#endif

	/** contains actor + filter hash, for interval-ticked filters (if passed) */
	TSet<uint32> IntervalFilterResults;
	/** contains actor + filter hash, for spawn filters (if passed) */
	TSet<uint32> SpawnFilterResults;

	/** Frame-accumulated spawn filter results, applied to SpawnFilterResults at the end of the frame */
	TSet<uint32> NewSpawnFilterPassResults;
	TSet<uint32> NewSpawnFilterRejectResults;

	// Contains filter index + its hash, map is re-populated each frame with interval filters which will be tick during it (see AddTickedIntervalFilter)
	TMap<uint32, uint32> IntervalFilterIndexAndHash;
};
