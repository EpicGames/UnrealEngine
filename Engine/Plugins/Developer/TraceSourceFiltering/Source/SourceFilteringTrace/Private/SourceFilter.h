// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

/** Simplified structure representing a DataSourceFilter and its state within the FSourceFilterManager */
struct FFilter
{
	/** Filter object */
	const class UDataSourceFilter* Filter;

	/** Filter's type-hash value*/
	uint32 FilterHash;

	/** Hash of the filter run (set) this filter is contained by */
	uint32 FilterSetHash;

	/** Index into FSourceFilterManager::(Last)FilterTickFrameIndices */
	uint32 TickFrameOffset;

	/** Cached Offset into the result (bit) array */
	uint32 ResultOffset;
	
	/** Interval, in frames, between whenever the filter should be applied to actors */
	uint8 TickInterval;
	
	/** Value this filter is expected to return for a given actor, in order for it to pass. (accumulated NOT operation) */
	uint8 bExpectedValue : 1;
	/** Whether or not the filter implementation is native, means we can call the native implemented function directly
	(rather than thunk) */
	uint8 bNative : 1;
	/** Whether or not this filter should only be applied during actor spawning, and reapplied whenever the filter set changes*/
	uint8 bOnSpawnOnly : 1;
	/** Whether or not this filter was marked to be able to run off the game thread */
	uint8 bCanRunAsynchronously : 1;
	/** Whether or not the actor, when passing this filter, should be regarded as passing for the entire filter-set */
	uint8 bEarlyOutPass : 1;
	/** Whether or not the actor, when failing to pass this filter, should be discarded for the entire filter-set */
	uint8 bEarlyOutDiscard : 1;
};