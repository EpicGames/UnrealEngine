// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceFilterCollection.h"
#include "DataSourceFilterSet.h"
#include "SourceFilterTrace.h"
#include "Async/Async.h"
#include "EmptySourceFilter.h"
#include "ObjectTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SourceFilterCollection)

template<typename T>
T* USourceFilterCollection::CreateNewFilter(UClass* Class /*= T::StaticClass()*/)
{
	T* NewFilter = NewObject<T>(this, Class, NAME_None, RF_Transactional);
	return NewFilter;
}

void USourceFilterCollection::OnObjectsReplaced(const TMap<UObject*, UObject*>& OldToNewInstanceMap)
{
	for (int32 FilterIndex = 0; FilterIndex < Filters.Num(); ++FilterIndex)
	{
		UDataSourceFilter* Filter = Filters[FilterIndex];
		if (UObject* const * NewObjectPtr = OldToNewInstanceMap.Find(Filter))
		{
			Filters.Insert(Cast<UDataSourceFilter>(*NewObjectPtr), FilterIndex);
			Filters.Remove(Filter);
		}
	}

	TArray<TObjectPtr<UDataSourceFilter>> KeyFilters;
	ChildToParent.GenerateKeyArray(KeyFilters);

	for (UDataSourceFilter* Filter : KeyFilters)
	{
		if (UObject* const * NewObjectPtr = OldToNewInstanceMap.Find(Filter))
		{
			TObjectPtr<UDataSourceFilterSet> ParentFilterSet = nullptr;			
			ChildToParent.RemoveAndCopyValue(Filter, ParentFilterSet);

			ChildToParent.Add(Cast<UDataSourceFilter>(*NewObjectPtr), ParentFilterSet);
		}
	}

	TArray<FObjectKey> Keys;
	FilterClassMap.GenerateKeyArray(Keys);

	for (const FObjectKey& ObjectKey : Keys)
	{
		if (UObject* const* NewObjectPtr = OldToNewInstanceMap.Find(ObjectKey.ResolveObjectPtr()))
		{
			FString ClassName;
			FilterClassMap.RemoveAndCopyValue(ObjectKey, ClassName);
			FilterClassMap.Add(*NewObjectPtr, ClassName);
		}
	}

	SourceFiltersUpdatedDelegate.Broadcast();
}

UDataSourceFilterSet* USourceFilterCollection::GetParentForFilter(UDataSourceFilter* Filter)
{
	return ChildToParent.FindChecked(Filter);
}

void USourceFilterCollection::AddClassFilter(TSubclassOf<AActor> InClass)
{
	ClassFilters.AddUnique({ FSoftClassPath(InClass), false });
	SourceFiltersUpdatedDelegate.Broadcast();
}

void USourceFilterCollection::RemoveClassFilter(TSubclassOf<AActor> InClass)
{
	ClassFilters.RemoveAll([InClass](FActorClassFilter FilteredClass)
	{
		return FilteredClass.ActorClass == FSoftClassPath(InClass);
	});
	SourceFiltersUpdatedDelegate.Broadcast();
}

void USourceFilterCollection::UpdateClassFilter(TSubclassOf<AActor> InClass, bool bIncludeDerivedClasses)
{
	FActorClassFilter* Class = ClassFilters.FindByPredicate([InClass](FActorClassFilter FilteredClass)
	{
		return FilteredClass.ActorClass == FSoftClassPath(InClass);
	});

	checkf(Class, TEXT("Invalid class provided"));
	if (Class)
	{
		Class->bIncludeDerivedClasses = bIncludeDerivedClasses;
		SourceFiltersUpdatedDelegate.Broadcast();
	}
}

void USourceFilterCollection::AddFilter(UDataSourceFilter* NewFilter)
{
	Filters.Add(NewFilter);
	ChildToParent.Add(NewFilter, nullptr);
	AddClassName(NewFilter);

	TRACE_FILTER_INSTANCE(NewFilter);
	SourceFiltersUpdatedDelegate.Broadcast();
}

UDataSourceFilter* USourceFilterCollection::AddFilterOfClass(const TSubclassOf<UDataSourceFilter>& FilterClass)
{
	checkf(FilterClass.Get(), TEXT("Cannot create filter using a null class"));

	UDataSourceFilter* NewFilter = CreateNewFilter<UDataSourceFilter>(FilterClass.Get());
	AddFilter(NewFilter);
	AddClassName(NewFilter);

	return NewFilter;
}

UDataSourceFilter* USourceFilterCollection::AddFilterOfClassToSet(const TSubclassOf<UDataSourceFilter>& FilterClass, UDataSourceFilterSet* FilterSet)
{
	checkf(FilterClass.Get() && FilterSet, TEXT("Cannot add filter using a null class, or null target set"));
	if (FilterSet)
	{
		UDataSourceFilter* NewFilter = CreateNewFilter<UDataSourceFilter>(FilterClass.Get());
		AddFilterToSet(NewFilter, FilterSet);

		return NewFilter;
	}

	return nullptr;
}

UDataSourceFilterSet* USourceFilterCollection::ConvertFilterToSet(UDataSourceFilter* ReplacedFilter, EFilterSetMode Mode)
{
	ensure(ReplacedFilter);

	UDataSourceFilterSet* NewFilterSet = CreateNewFilter<UDataSourceFilterSet>();
	TRACE_FILTER_SET(NewFilterSet);
	AddClassName(NewFilterSet);

	NewFilterSet->SetFilterMode(Mode);

	ReplaceFilter(ReplacedFilter, NewFilterSet);
	AddFilterToSet(ReplacedFilter, NewFilterSet);

	return NewFilterSet;
}

UDataSourceFilterSet* USourceFilterCollection::MakeFilterSet(UDataSourceFilter* FilterOne, UDataSourceFilter* FilterTwo, EFilterSetMode Mode)
{
	ensure(FilterOne && FilterTwo);

	UDataSourceFilterSet* NewFilterSet = CreateNewFilter<UDataSourceFilterSet>();
	TRACE_FILTER_SET(NewFilterSet);
	AddClassName(NewFilterSet);

	NewFilterSet->SetFilterMode(Mode);

	ReplaceFilter(FilterOne, NewFilterSet);

	AddFilterToSet(FilterOne, NewFilterSet);
	MoveFilter(FilterTwo, NewFilterSet);

	return NewFilterSet;
}

void USourceFilterCollection::SetFilterSetMode(UDataSourceFilterSet* FilterSet, EFilterSetMode Mode)
{
	if (FilterSet)
	{
		if (FilterSet->GetFilterSetMode() != Mode)
		{
			FilterSet->Modify();
			FilterSet->SetFilterMode(Mode);

			SourceFiltersUpdatedDelegate.Broadcast();
		}
	}
}

UDataSourceFilterSet* USourceFilterCollection::MakeEmptyFilterSet(EFilterSetMode Mode)
{
	UDataSourceFilterSet* NewFilterSet = CreateNewFilter<UDataSourceFilterSet>();
	TRACE_FILTER_SET(NewFilterSet);
	AddClassName(NewFilterSet);

	NewFilterSet->SetFilterMode(Mode);

	Filters.Add(NewFilterSet);
	ChildToParent.FindOrAdd(NewFilterSet) = nullptr;

	SourceFiltersUpdatedDelegate.Broadcast();

	return NewFilterSet;
}

void USourceFilterCollection::AddFiltersFromPreset(const TArray<FString>& ClassNames, const TMap<int32, int32>& ChildToParentIndices)
{
	AsyncTask(ENamedThreads::GameThread,
		[this, ClassNames, ChildToParentIndices]()
	{
		const UEnum* EnumPtr = FindObject<UEnum>(nullptr, TEXT("/Script/SourceFilteringCore.EFilterSetMode"), EFindObjectFlags::ExactClass);

		TArray<UDataSourceFilter*> CreatedFilters;
		TArray<bool> MovedFlags;

		CreatedFilters.SetNumZeroed(ClassNames.Num());
		MovedFlags.SetNumZeroed(ClassNames.Num());

		for (const TPair<int32, int32>& ChildToParentPair : ChildToParentIndices)
		{
			const int32 FilterIndex = ChildToParentPair.Key;
			const int64 Value = EnumPtr->GetValueByNameString(ClassNames[FilterIndex]);
			if (Value != INDEX_NONE)
			{
				// This is a set, so create one
				CreatedFilters[FilterIndex] = MakeEmptyFilterSet((EFilterSetMode)Value);
			}
			else
			{
#if SOURCE_FILTER_TRACE_ENABLED
				// This is a filter, so create it according to the class name				
				if (UClass* Class = FSourceFilterTrace::RetrieveClassByName(ClassNames[FilterIndex]))
				{
					if (UDataSourceFilter* Filter = CreateNewFilter<UDataSourceFilter>(Class))
					{
						CreatedFilters[FilterIndex] = Filter;
						AddFilter(Filter);
					}
				}
#endif // SOURCE_FILTER_TRACE_ENABLED
			}

			// In case we were unable to retrieve a Filter instance its class replace it with an empty 'stub' filter instance, indicating to the user that it, and its class, is missing
			if (CreatedFilters[FilterIndex] == nullptr)
			{
				if (UEmptySourceFilter* Filter = CreateNewFilter<UEmptySourceFilter>())
				{
					TRACE_FILTER_CLASS(UEmptySourceFilter::StaticClass());
					CreatedFilters[FilterIndex] = Filter;

					Filter->MissingClassName = ClassNames[FilterIndex];
					AddFilter(Filter);
				}

			}

			if (CreatedFilters[FilterIndex] != nullptr && ChildToParentPair.Value == INDEX_NONE)
			{
				MovedFlags[FilterIndex] = true;
			}
		}

		// Regenerate the parent child relationship from the preset
		bool bAnyMoved = true;
		while (bAnyMoved)
		{
			bAnyMoved = false;
			for (const TPair<int32, int32>& ChildToParentPair : ChildToParentIndices)
			{
				if (ChildToParentPair.Value != INDEX_NONE && !MovedFlags[ChildToParentPair.Key] && MovedFlags[ChildToParentPair.Value])
				{
					MoveFilter(CreatedFilters[ChildToParentPair.Key], CastChecked<UDataSourceFilterSet>(CreatedFilters[ChildToParentPair.Value]));
					MovedFlags[ChildToParentPair.Key] = true;
					bAnyMoved = true;
				}
			}
		}
	});
}

void USourceFilterCollection::RemoveFilter(UDataSourceFilter* ToRemoveFilter)
{
	TObjectPtr<UDataSourceFilterSet> OuterFilterSet = nullptr;
	ChildToParent.RemoveAndCopyValue(ToRemoveFilter, OuterFilterSet);
	
	// In case of a set, also remove children 
	if (UDataSourceFilterSet* FilterSet = Cast<UDataSourceFilterSet>(ToRemoveFilter))
	{
		FilterSet->Modify();

		// Also remove contained children
		for (UDataSourceFilter* Filter : FilterSet->Filters)
		{
			RemoveFilterRecursive(Filter);
		}

		FilterSet->Filters.Empty();
	}

	TRACE_FILTER_OPERATION(ToRemoveFilter, ESourceActorFilterOperation::RemoveFilter, TRACE_FILTER_IDENTIFIER(Cast<UDataSourceFilterSet>(ToRemoveFilter)));

	if (OuterFilterSet)
	{
		OuterFilterSet->Modify();
		RemoveFilterFromSet(ToRemoveFilter, OuterFilterSet);
	}
	else
	{
		Filters.RemoveSingle(ToRemoveFilter);
	}

	SourceFiltersUpdatedDelegate.Broadcast();
}

void USourceFilterCollection::RemoveFilterFromSet(UDataSourceFilter* ToRemoveFilter, UDataSourceFilterSet* FilterSet)
{
	FilterSet->Filters.RemoveSingle(ToRemoveFilter);

	// in case set is empty, remove it
	if (FilterSet->Filters.Num() == 0)
	{
		RemoveFilter(FilterSet);
	}
}

void USourceFilterCollection::RemoveFilterRecursive(UDataSourceFilter* ToRemoveFilter)
{
	ChildToParent.Remove(ToRemoveFilter);

	if (UDataSourceFilterSet* FilterSet = Cast<UDataSourceFilterSet>(ToRemoveFilter))
	{
		for (UDataSourceFilter* Filter : FilterSet->Filters)
		{
			RemoveFilterRecursive(Filter);
		}
	}

	TRACE_FILTER_OPERATION(ToRemoveFilter, ESourceActorFilterOperation::RemoveFilter, TRACE_FILTER_IDENTIFIER(Cast<UDataSourceFilterSet>(ToRemoveFilter)));
}

void USourceFilterCollection::AddFilterToSet(UDataSourceFilter* Filter, UDataSourceFilterSet* FilterSet)
{
	if (FilterSet)
	{
		FilterSet->Modify();

		FilterSet->Filters.Add(Filter);
		ChildToParent.Add(Filter, FilterSet);
		AddClassName(Filter);

		TRACE_FILTER_INSTANCE(Filter);
		TRACE_FILTER_OPERATION(Filter, ESourceActorFilterOperation::MoveFilter, TRACE_FILTER_IDENTIFIER(FilterSet));
		SourceFiltersUpdatedDelegate.Broadcast();
	}
}

void USourceFilterCollection::ReplaceFilter(UDataSourceFilter* Destination, UDataSourceFilter* Source)
{
	TRACE_FILTER_OPERATION(Source, ESourceActorFilterOperation::ReplaceFilter, TRACE_FILTER_IDENTIFIER(Destination));

	TObjectPtr<UDataSourceFilterSet> OuterFilterSet = nullptr;
	ChildToParent.RemoveAndCopyValue(Destination, OuterFilterSet);

	if (OuterFilterSet)
	{
		OuterFilterSet->Modify();

		const int32 FilterEntryIndex = OuterFilterSet->Filters.IndexOfByKey(Destination);
		OuterFilterSet->Filters[FilterEntryIndex] = Source;
	}
	else
	{
		const int32 FilterEntryIndex = Filters.IndexOfByKey(Destination);
		Filters[FilterEntryIndex] = Source;
	}

	ChildToParent.FindOrAdd(Source) = OuterFilterSet;

	SourceFiltersUpdatedDelegate.Broadcast();
}

void USourceFilterCollection::MoveFilter(UDataSourceFilter* Filter, UDataSourceFilterSet* Destination)
{
	TRACE_FILTER_OPERATION(Filter, ESourceActorFilterOperation::MoveFilter, Destination ? TRACE_FILTER_IDENTIFIER(Destination) : 0);

	TObjectPtr<UDataSourceFilterSet>& FilterParent = ChildToParent.FindChecked(Filter);

	if (FilterParent)
	{
		RemoveFilterFromSet(Filter, FilterParent);
	}
	else
	{
		// Currently top level filter
		Filters.RemoveSingle(Filter);
	}

	if (Destination)
	{
		Destination->Filters.Add(Filter);
	}
	else
	{
		// Make top level filter
		Filters.Add(Filter);
	}

	FilterParent = Destination;

	SourceFiltersUpdatedDelegate.Broadcast();
}

void USourceFilterCollection::SetFilterState(UDataSourceFilter* Filter, bool bEnabled)
{
	if (Filter)
	{
		Filter->Modify();
		Filter->SetEnabled(bEnabled);

		SourceFiltersUpdatedDelegate.Broadcast();
	}
}

void USourceFilterCollection::Reset()
{
	TArray<UDataSourceFilter*> FiltersToRemove = Filters;
	for (UDataSourceFilter* Filter : FiltersToRemove)
	{
		RemoveFilter(Filter);
	}
	FilterClasses.Empty();
	FilterClassMap.Empty();
	ClassFilters.Empty();

	SourceFiltersUpdatedDelegate.Broadcast();
}

void USourceFilterCollection::GetFlatFilters(TArray<TObjectPtr<UDataSourceFilter>>& OutFilters)
{
	ChildToParent.GenerateKeyArray(OutFilters);
}

void USourceFilterCollection::CopyData(USourceFilterCollection* OtherCollection)
{
	Filters.Empty();
	ChildToParent.Empty();
	FilterClasses = OtherCollection->FilterClasses;
	ClassFilters = OtherCollection->ClassFilters;

	int32 FilterOffset = 0;
	for (UDataSourceFilter* Filter : OtherCollection->Filters)
	{
		UDataSourceFilter* FilterCopy = RecursiveCopyFilter(Filter, FilterOffset);
		Filters.Add(FilterCopy);
		ChildToParent.Add(FilterCopy, nullptr);
		AddClassName(FilterCopy);
	}
	FilterClasses.Empty();

	// Remove any filter classes that are unable to be loaded
	ClassFilters.RemoveAll([](FActorClassFilter& Filter)
	{
		return Filter.ActorClass.TryLoadClass<AActor>() == nullptr;
	});

	SourceFiltersUpdatedDelegate.Broadcast();
}

UDataSourceFilter* USourceFilterCollection::RecursiveCopyFilter(UDataSourceFilter* Filter, int32& FilterOffset)
{
	UDataSourceFilter* FilterCopy = nullptr;
	if (Filter)
	{
		FilterCopy = DuplicateObject(Filter, this);
		++FilterOffset;

		if (UDataSourceFilterSet* FilterSet = Cast<UDataSourceFilterSet>(FilterCopy))
		{
			TArray<UDataSourceFilter*> NewChildFilters;
			for (UDataSourceFilter* ChildFilter : FilterSet->Filters)
			{
				UDataSourceFilter* ChildFilterCopy = RecursiveCopyFilter(ChildFilter, FilterOffset);
				ChildToParent.Add(ChildFilterCopy, FilterSet);
				NewChildFilters.Add(ChildFilterCopy);

				AddClassName(ChildFilterCopy);
			}

			FilterSet->Filters = NewChildFilters;
		}
	}
	else
	{
		UEmptySourceFilter* EmptyFilter = CreateNewFilter<UEmptySourceFilter>();
		if (FilterClasses.IsValidIndex(FilterOffset))
		{
			EmptyFilter->MissingClassName = FilterClasses[FilterOffset];
		}

		FilterCopy = EmptyFilter;
		++FilterOffset;
	}

	return FilterCopy;
}

void USourceFilterCollection::Serialize(FArchive& Ar)
{
	if (Ar.IsSaving() && !Ar.IsTransacting())
	{
		FilterClasses.Empty();

		// Regenerate the flat list of class names, corresponding to the entries in Filters, main purpose is to be able to surface missing Filter class names to the user
		for (UDataSourceFilter* Filter : Filters)
		{
			RecursiveRetrieveFilterClassNames(Filter);
		}
	}

	Super::Serialize(Ar);

	if (Ar.IsLoading() && !Ar.IsTransacting())
	{
		FilterClassMap.Empty();
		
		for (UDataSourceFilter* Filter : Filters)
		{
			RecursiveGenerateFilterClassNames(Filter);
		}
	}
}

void USourceFilterCollection::AddClassName(UDataSourceFilter* Filter)
{
	FilterClassMap.Add(Filter, Filter->GetClass()->GetName());
}

FSimpleMulticastDelegate& USourceFilterCollection::GetSourceFiltersUpdated()
{
	return SourceFiltersUpdatedDelegate;
}

void USourceFilterCollection::RecursiveGenerateFilterClassNames(UDataSourceFilter* Filter)
{
	AddClassName(Filter);

	if ( UDataSourceFilterSet* FilterSet = Cast<UDataSourceFilterSet>(Filter))
	{
		for (UDataSourceFilter* ChildFilter : FilterSet->Filters)
		{
			RecursiveGenerateFilterClassNames(ChildFilter);
		}
	}
}

void USourceFilterCollection::RecursiveRetrieveFilterClassNames(UDataSourceFilter* Filter)
{
	if (Filter)
	{
		const FString ClassName = FilterClassMap.FindChecked(Filter);
		FilterClasses.Add(ClassName);

		if (UDataSourceFilterSet* FilterSet = Cast<UDataSourceFilterSet>(Filter))
		{
			/* Any child filters will be added in-line, meaning the list will look as such:
				TopLevelFilterSetA_Class
					ChildOneFilter_Class
					ChildTwoFilter_Class
				TopLevelFilter_Class
				etc.
			*/
			for (UDataSourceFilter* ChildFilter : FilterSet->Filters)
			{
				RecursiveRetrieveFilterClassNames(ChildFilter);
			}
		}
	}
}
