// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceFilterManager.h"

#include "DataSourceFilter.h"
#include "EngineUtils.h"
#include "Async/ParallelFor.h"
#include "Misc/CoreDelegates.h"
#include "ObjectTrace.h"
#include "HAL/LowLevelMemStats.h"

#include "Stats/StatsTrace.h"
#include "TraceSourceFiltering.h"
#include "DrawDebugHelpers.h"
#include "TraceSourceFilteringSettings.h"
#include "SourceFilteringAsyncTasks.h"

#define USE_NON_THREADSAFE_TRACE_FILTER_API (UE_TRACE_ENABLED && 1)

DEFINE_LOG_CATEGORY_STATIC(SourceFilterManager, Display, Display);

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Total Actors"), STAT_SourceFilterManager_AllActors, STATGROUP_SourceFilterManager);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Early Discarded Actors"), STAT_SourceFilterManager_EarlyDiscardedActors, STATGROUP_SourceFilterManager);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Early Passed Actors"), STAT_SourceFilterManager_EarlyPassedActors, STATGROUP_SourceFilterManager);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Evaluated Actors"), STAT_SourceFilterManager_EvaluatedActors, STATGROUP_SourceFilterManager);

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Passed Actors"), STAT_SourceFilterManager_PassedActors, STATGROUP_SourceFilterManager);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Rejected Actors"), STAT_SourceFilterManager_RejectedActors, STATGROUP_SourceFilterManager);

#if USE_NON_THREADSAFE_TRACE_FILTER_API 
static const bool bForceThreadSafe = false;
#else
static const bool bForceThreadSafe = true;
#endif

FSourceFilterManager::FSourceFilterManager(UWorld* InWorld) : World(InWorld), bAreTickFunctionsRegistered(false), ActorCollector(World, {}), FilterSetup(FSourceFilterSetup::GetFilterSetup())
{
	auto ActorSpawnLambda = FOnActorSpawned::FDelegate::CreateLambda([this](AActor* InActor)
	{ 
		// Apply filters if the world is marked as trace-able itself
		if (CAN_TRACE_OBJECT(World))
		{	
			// Apply high level class filters
			bool bClassFilterResult = true;
			if (FilterSetup.HasAnyClassFilters())
			{
				bClassFilterResult = ApplyClassFilterToActor(InActor);
			}

			// Apply spawn-only filters
			if (FilterSetup.HasAnySpawnFilters())
			{
				if (bClassFilterResult)
				{
					DiscardedActorHashes.Remove(GetTypeHash(InActor));
					PassedActorsHashes.Remove(GetTypeHash(InActor));
					ApplySpawnFiltersToActor(InActor);
				}
			}
			// Otherwise if no other filters are setup either, mark the actor as traceable
			else
			{
				if (!FilterSetup.HasAnyFilters())
				{
					SET_OBJECT_TRACEABLE(InActor, bClassFilterResult);
				}
			}
		}
	});

	ActorSpawningDelegateHandle = World->AddOnActorSpawnedHandler(ActorSpawnLambda);
	PreActorSpawningDelegateHandle = World->AddOnActorPreSpawnInitialization(ActorSpawnLambda);

	// Ensure that we process all actors whenever the world finishes initialization
	World->OnActorsInitialized.AddLambda([this](const UWorld::FActorsInitializedParams& Params)
	{
		check(Params.World == World);
		OnFilterSetupChanged();
	});

	// Register call-back to filtering settings
	Settings = FTraceSourceFiltering::Get().GetSettings();
	Settings->GetOnSourceFilteringSettingsChanged().AddRaw(this, &FSourceFilterManager::OnSourceFilteringSettingsChanged);

	// Register call-back to filter setup to responds to state changes
	FilterSetup.GetFilterSetupUpdated().AddRaw(this, &FSourceFilterManager::OnFilterSetupChanged);
	OnFilterSetupChanged();
}

FSourceFilterManager::~FSourceFilterManager()
{
	if (World)
	{
		World->RemoveOnActorSpawnedHandler(ActorSpawningDelegateHandle);
		World->RemoveOnActorPreSpawnInitialization(PreActorSpawningDelegateHandle);
	}

	FilterSetup.GetFilterSetupUpdated().RemoveAll(this);

	WaitForAsyncTasks();

	if (IsValidRef(DrawTask) && !DrawTask->IsComplete())
	{
		// Mark drawing task as finished (would otherwise have been executed on GT)
		DrawTask->DispatchSubsequents(ENamedThreads::GameThread);
	}

	if (Settings)
	{
		Settings->GetOnSourceFilteringSettingsChanged().RemoveAll(this);
	}

	UnregisterTickFunctions();
}

template<EFilterType FilterType>
void FSourceFilterManager::ApplyFilters()
{
	if (CAN_TRACE_OBJECT(World))
	{
		const int32 NumActors = ActorCollector.NumActors();
		// We can cache the root filter set result whenever we are running either only GT or async filters (as their results do not need to be combined)
		const bool bCacheResult = !FilterSetup.HasGameThreadFilters() == FilterSetup.HasAsyncFilters();

		if (FilterType == EFilterType::GameThreadFilters)
		{
			ActorHashes.SetNumZeroed(NumActors, EAllowShrinking::No);

			const FFilterSet& RootSet = FilterSetup.GetRootSet();
			check(RootSet.bContainsGameThreadFilter);

			const uint32 Loops = FMath::CeilToInt((float)NumActors / 32.0f);
			ParallelFor(Loops,
				[&](int32 LoopIndex)
			{
				const int32 Start = LoopIndex * 32;
				const int32 End = FMath::Min((LoopIndex + 1) * 32, NumActors);
				for (FFilteredActorIterator ActorIt = ActorCollector.GetIterator(Start, End - Start); ActorIt; ++ActorIt)
				{
					const AActor* Actor = *ActorIt;
					const uint32 ActorHash = GetTypeHash(Actor);
					const uint32 ActorIndex = ActorIt.Index();
					if (Actor && !PassedActorsHashes.Contains(ActorHash) && !DiscardedActorHashes.Contains(ActorHash))
					{
						const FHashSet* PassSets = PassedFilterSets.Find(ActorHash);
						const FHashSet* DiscardSets = DiscardedFilterSets.Find(ActorHash);

						ActorHashes[ActorIndex] = ActorHash;

						FActorFilterInfo ActorInfo = { Actor, ActorHash, PassSets, DiscardSets, ActorIndex };

						const bool bResult = ApplyFilterSetToActor<FilterType>(RootSet, ActorInfo);

						if (bCacheResult)
						{
							ResultCache.CacheFilterSetResult<FilterType>(RootSet, ActorInfo, bResult);
						}
					}
				}
			});
		}
		else if (FilterType == EFilterType::AsynchronousFilters)
		{
			const FFilterSet& RootSet = FilterSetup.GetRootSet();
			check(RootSet.bContainsAsyncFilter);

			for (FFilteredActorIterator ActorIt = ActorCollector.GetIterator(); ActorIt; ++ActorIt)
			{
				const AActor* Actor = *ActorIt;
				const uint32 ActorHash = GetTypeHash(Actor);
				if (Actor && !PassedActorsHashes.Contains(ActorHash) && !DiscardedActorHashes.Contains(ActorHash))
				{
					const FHashSet* PassSets = PassedFilterSets.Find(ActorHash);
					const FHashSet* DiscardSets = DiscardedFilterSets.Find(ActorHash);
					FActorFilterInfo ActorInfo = { Actor, ActorHash, PassSets, DiscardSets, ActorIt.Index() };
					const bool bResult = ApplyFilterSetToActor<FilterType>(RootSet, ActorInfo);

					if (bCacheResult)
					{
						ResultCache.CacheFilterSetResult<FilterType>(RootSet, ActorInfo, bResult);
					}
				}
			}
		}
	}
}

void FSourceFilterManager::ApplyGameThreadFilters()
{
	ApplyFilters<EFilterType::GameThreadFilters>();
}

void FSourceFilterManager::ApplyAsyncFilters()
{
	ApplyFilters<EFilterType::AsynchronousFilters>();
}

void FSourceFilterManager::ApplySpawnFilters()
{
	DiscardedActorHashes.Empty();
	PassedActorsHashes.Empty();
	if (CAN_TRACE_OBJECT(World))
	{	
		for (FFilteredActorIterator ActorIt = ActorCollector.GetIterator(); ActorIt; ++ActorIt)
		{
			if (const AActor* Actor = *ActorIt)
			{
				ApplySpawnFiltersToActor(Actor);
			}
		}
	}
}

template<EFilterType FilterType>
bool FSourceFilterManager::ApplyFilterSetToActor(const FFilterSet& FilterSet, const FActorFilterInfo& ActorInfo)
{
	// Check whether or not this filter set can be earlied-out
	if (ActorInfo.PassSets && ActorInfo.PassSets->Contains(FilterSet.FilterSetHash))
	{
		return true;
	}

	if (ActorInfo.DiscardSets && ActorInfo.DiscardSets->Contains(FilterSet.FilterSetHash))
	{
		return false;
	}

	const int32 NumFilters = FilterSet.FilterEntries.Num();
	int32 ProcessedFilters = 0;

	const bool bIsANDFilterSet = FilterSet.Mode == EFilterSetMode::AND;

	// Loop over all contained filters
	for (int32 FilterIndex = 0; FilterIndex < NumFilters; ++FilterIndex)
	{
		const FFilter& FilterEntry = FilterSet.FilterEntries[FilterIndex];
		const UDataSourceFilter* Filter = FilterEntry.Filter;
		const uint32& FilterHash = FilterEntry.FilterHash;

		bool bPassesFilter = false;
			   
		
		const bool bShouldTryApplying = 
		(
			(FilterType == EFilterType::GameThreadFilters && !FilterEntry.bCanRunAsynchronously) ||
			(FilterType == EFilterType::AsynchronousFilters && FilterEntry.bCanRunAsynchronously) ||
			FilterEntry.bOnSpawnOnly
		);

		if (bShouldTryApplying)
		{
			const bool bShouldTickForThisFrame = FilterEntry.TickInterval == 1 ? true : WorldData.IntervalFilterShouldTick[FilterEntry.TickFrameOffset];

			// In case this is a Spawn Only filter or it is not required to tick this frame, return the previously cached value instead
			if (FilterEntry.bOnSpawnOnly || !bShouldTickForThisFrame)
			{
				bPassesFilter = ResultCache.RetrieveCachedResult(FilterEntry, ActorInfo);
			}
			else
			{
				bPassesFilter = !!FilterEntry.bNative ? Filter->DoesActorPassFilter_Internal(ActorInfo.Actor) : Filter->DoesActorPassFilter(ActorInfo.Actor);
				// Compare filter result with expected value
				bPassesFilter = (bPassesFilter == !!FilterEntry.bExpectedValue);

				// Cache result for filter and actor combination
				ResultCache.CacheFilterResult(FilterEntry, ActorInfo, bPassesFilter);
			}

			// In case this filter did pass, and is marked as early-out do so
			if (FilterEntry.bEarlyOutPass && bPassesFilter)
			{
				return true;
			}

			// In case this filter did not pass, and is marked as early-out do so
			if (FilterEntry.bEarlyOutDiscard && !bPassesFilter)
			{
				return false;
			}

			++ProcessedFilters;
		}
	}

	// Apply contained child filter sets
	for (const FFilterSet& ChildFilterSet : FilterSet.ChildFilterSets)
	{
		const bool bChildSetResult = ApplyFilterSetToActor<FilterType>(ChildFilterSet, ActorInfo);

		if (FilterSetup.RequiresApplyingFilters())
		{
			ResultCache.CacheFilterSetResult<FilterType>(ChildFilterSet, ActorInfo, bChildSetResult);
		}

		if (bChildSetResult)
		{
			// If the child run passed, and this is an OR set we can early out (true)
			if (!bIsANDFilterSet)
			{
				return true;
			}
		}
		else
		{
			// If the child run did not pass, and this is an AND set we can early out (false)
			if (bIsANDFilterSet)
			{
				return false;
			}
		}
	}

	// Return whether or not we processed any filters or child filters, meaning none earlied (false) out from the AND operation or (true) for the OR operation
	return (ProcessedFilters > 0 || FilterSet.ChildFilterSets.Num()) && bIsANDFilterSet;
}

void FSourceFilterManager::ApplySpawnFiltersToActor(const AActor* Actor)
{
	const uint32 ActorHash = GetTypeHash(Actor);

	for (const FFilter& FilterEntry : FilterSetup.GetSpawnFilters())
	{
		const bool bPassesFilter = [FilterEntry, Actor]()
		{
			if (FilterEntry.bNative)
			{
				return FilterEntry.Filter->DoesActorPassFilter_Internal(Actor) == FilterEntry.bExpectedValue;
			}

			return FilterEntry.Filter->DoesActorPassFilter(Actor) == FilterEntry.bExpectedValue;
		}();
			   
		ResultCache.CacheSpawnFilterResult(FilterEntry.FilterHash, ActorHash, bPassesFilter);

		// If the result of applying the spawn filter causes this actor to be rejected or pass for the entire set mark its hash as such
		if (!bPassesFilter && FilterEntry.bEarlyOutDiscard)
		{
			DiscardedActorHashes.Add(ActorHash);
			break;
		}
		else if (bPassesFilter && FilterEntry.bEarlyOutPass)
		{	
			PassedActorsHashes.Add(ActorHash);
			break;
		}

		// Collate any filter-sets that can be skipped for this actor 
		if (!bPassesFilter)
		{
			DiscardedFilterSets.FindOrAdd(ActorHash).Append(FilterSetup.GetSpawnFilterDiscardableSets(FilterEntry));
		}
		else if (bPassesFilter)
		{
			PassedFilterSets.FindOrAdd(ActorHash).Append(FilterSetup.GetSpawnFilterSkippableSets(FilterEntry));
		}
	}

	// If there are only spawn filters, apply results directly
	const bool bApplyResults = FilterSetup.HasOnlySpawnFilters();
	if (bApplyResults)
	{
		ResultCache.ProcessSpawnFilterResults();

		// We should run the filter sets here, as we will have all of the results from the spawn-only filters
		FActorFilterInfo ActorInfo = { Actor, ActorHash, nullptr, nullptr };
		const bool bResult = ApplyFilterSetToActor<EFilterType::AsynchronousFilters>(FilterSetup.GetRootSet(), ActorInfo);
		
#if TRACE_FILTERING_ENABLED
		FFilterTraceScopeLock Lock;
		ApplyFilterResultsToActor(ActorInfo);		
#endif // TRACE_FILTERING_ENABLED
	}
}

void FSourceFilterManager::OnFilterSetupChanged()
{
	// Wait until pending async tasks have finished
	WaitForAsyncTasks();	
	
	// Reset all resident filtering data
	ResetFilterData();
	
	FilterSetup.PopulatePerWorldData(WorldData);
	ActorCollector.UpdateFilterClasses(FilterSetup.GetActorFilters());
	
	// Unregister tick functions, might not be needed according to filtering state
	UnregisterTickFunctions();

	// Spawn only filters will be applied directly if there aren't any regular filters
	if (FilterSetup.RequiresApplyingFilters())
	{
		RegisterTickFunctions();
	}

	if (FilterSetup.HasAnySpawnFilters())
	{
		ApplySpawnFilters();
	}

	if (!FilterSetup.RequiresApplyingFilters() || FilterSetup.HasAnyClassFilters())
	{
		// Should mark all current actors as traceable, according to class filter
		if (CAN_TRACE_OBJECT(World))
		{
			for (TActorIterator<AActor> It(World); It; ++It)
			{
				if (const AActor* Actor = *It)
				{
					const bool bCanTrace = FilterSetup.HasAnyClassFilters() ? ApplyClassFilterToActor(Actor) : true;
					const bool bCouldTrace = CAN_TRACE_OBJECT(Actor);
					if (bCanTrace != bCouldTrace)
					{
						SET_OBJECT_TRACEABLE(Actor, bCanTrace);
						// In case this actor would be traced for the first time, its object data should be traced out as well
						if (!bCouldTrace)
						{
							TRACE_OBJECT(Actor);
						}
					}
				}
			}
		}
	}

	OnSourceFilteringSettingsChanged();
}

void FSourceFilterManager::OnSourceFilteringSettingsChanged()
{
	// (Un)register tick functions according to user filtering settings
	const bool bShouldRegister = !bAreTickFunctionsRegistered && Settings->bDrawFilteringStates;
	const bool bShouldUnregister = bAreTickFunctionsRegistered && !Settings->bDrawFilteringStates && !FilterSetup.RequiresApplyingFilters();

	if (bShouldRegister)
	{
		RegisterTickFunctions();
	}

	if (bShouldUnregister)
	{
		UnregisterTickFunctions();
	}
}

void FSourceFilterManager::ResetFilterData()
{
	LLM_SCOPED_SINGLE_STAT_TAG(SourceFilterManager);

	ResultCache.Reset();

	DiscardedActorHashes.Empty();
	PassedActorsHashes.Empty();	
	PassedFilterSets.Empty();
	DiscardedFilterSets.Empty();
}


void FSourceFilterManager::ApplyFilterResults()
{
	if (CAN_TRACE_OBJECT(World))
	{	
		SET_DWORD_STAT(STAT_SourceFilterManager_AllActors, Statistics.TotalActors);
		SET_DWORD_STAT(STAT_SourceFilterManager_EvaluatedActors, Statistics.EvaluatedActors);
		SET_DWORD_STAT(STAT_SourceFilterManager_RejectedActors, Statistics.RejectedActors);
		SET_DWORD_STAT(STAT_SourceFilterManager_PassedActors, Statistics.PassedActors);
		SET_DWORD_STAT(STAT_SourceFilterManager_EarlyDiscardedActors, Statistics.EarlyRejectedActors);
		SET_DWORD_STAT(STAT_SourceFilterManager_EarlyPassedActors, Statistics.EarlyPassedActors);
		
		TRACE_OBJECT(World);

		FMemory::Memzero(Statistics);

		ensure(FilterSetup.RequiresApplyingFilters());
		
#if TRACE_FILTERING_ENABLED		
		FFilterTraceScopeLock Lock;
		for (FFilteredActorIterator ActorIt = ActorCollector.GetIterator(); ActorIt; ++ActorIt)
		{
			const AActor* Actor = *ActorIt;
			if (Actor)
			{
				++Statistics.TotalActors;

				FActorFilterInfo ActorInfo = { Actor, GetTypeHash(Actor), nullptr, nullptr, ActorIt.Index() };
				ApplyFilterResultsToActor(ActorInfo);
			}
		}
#endif // TRACE_FILTERING_ENABLED

		ResultCache.ProcessIntervalFilterResults(ActorHashes);
	}

}

void FSourceFilterManager::ApplyFilterResultsToActor(const FActorFilterInfo& ActorInfo)
{
	bool bPassesFilters = false;
	
	// Check the hash against any previous Spawn-Only filter results
	const uint32 ActorHash = ActorInfo.ActorHash;
	const bool bEarlyOutPassed = PassedActorsHashes.Contains(ActorHash);
	const bool bEarlyOutDiscarded = DiscardedActorHashes.Contains(ActorHash);

	if (bEarlyOutPassed)
	{
		++Statistics.EarlyPassedActors;
		bPassesFilters = true;
	}
	else if (bEarlyOutDiscarded)
	{
		++Statistics.EarlyRejectedActors;
		bPassesFilters = false;
	}
	else
	{		
		++Statistics.EvaluatedActors;
		bPassesFilters = DetermineActorFilteringState(ActorInfo);
	}

	Statistics.PassedActors += !!bPassesFilters;
	Statistics.RejectedActors += !bPassesFilters;


#if TRACE_FILTERING_ENABLED
	const bool bCouldTrace = FTraceFilter::IsObjectTraceable<bForceThreadSafe>(ActorInfo.Actor);
	if (bCouldTrace != bPassesFilters)
	{
		FTraceFilter::SetObjectIsTraceable<bForceThreadSafe>(ActorInfo.Actor, bPassesFilters);
		// In case this actor would be traced for the first time, its object data should be traced out as well
		if (!bCouldTrace && bPassesFilters)
		{
			TRACE_OBJECT(ActorInfo.Actor);
		}
	}	
#endif // TRACE_FILTERING_ENABLED
}

bool FSourceFilterManager::DetermineActorFilteringState(const FActorFilterInfo& ActorInfo) const
{
	// Process results for the root filter-set
	const FFilterSet& RootSet = FilterSetup.GetRootSet();
	const bool bRootSetCached = (!FilterSetup.HasGameThreadFilters() == FilterSetup.HasAsyncFilters());
	if (bRootSetCached)
	{
		return ResultCache.RetrieveFilterSetResult(RootSet, ActorInfo);
	}
	else
	{
		// If it was not stored (running both GT and async filters), we'll need to process the root filter set manually
		for (const FFilter& Filter : RootSet.FilterEntries)
		{
			if (ResultCache.RetrieveCachedResult(Filter, ActorInfo))
			{
				return true;
			}
		}

		for (const FFilterSet& ChildFilterSet : RootSet.ChildFilterSets)
		{
			if (ResultCache.RetrieveFilterSetResult(ChildFilterSet, ActorInfo))
			{
				return true;
			}
		}
	}

	return false;
}

bool FSourceFilterManager::ApplyClassFilterToActor(const AActor* Actor)
{
	for (const FActorClassFilter& ClassFilter : FilterSetup.GetActorFilters())
	{
		const UClass* ActorClass = Actor->GetClass();
		const UClass* FilterClass = ClassFilter.ActorClass.TryLoadClass<AActor>();
		if (ActorClass == FilterClass || (ClassFilter.bIncludeDerivedClasses && ActorClass->IsChildOf(FilterClass)))
		{
			return true;
		}
	}

	return false;
}

void FSourceFilterManager::DrawFilterResults()
{
	if (CAN_TRACE_OBJECT(World))
	{
		if (!FilterSetup.RequiresApplyingFilters())
		{
			ActorCollector.CollectActors();
		}

		for (FFilteredActorIterator ActorIt = ActorCollector.GetIterator(); ActorIt; ++ActorIt)
		{
			DrawFilterResults(*ActorIt);
		}
	}
}

void FSourceFilterManager::DrawFilterResults(const AActor* Actor)
{
	// Debug-purpose drawing, allowing users to see impact of Filter set
	if (Settings->bDrawFilteringStates && (World->WorldType == EWorldType::Game || World->WorldType == EWorldType::PIE))
	{
		const bool bPassesFilters = CAN_TRACE_OBJECT(Actor);

		FVector Origin, Extent;
		Actor->GetActorBounds(true, Origin, Extent);
		if (Settings->bDrawOnlyPassingActors)
		{
			if (bPassesFilters)
			{
				DrawDebugBox(World, Origin, Extent, FQuat::Identity, FColor::Green, false, -1.f, 0, 1.f);
			}
		}
		else
		{
			DrawDebugBox(World, Origin, Extent, FQuat::Identity, bPassesFilters ? FColor::Green : FColor::Red, false, -1.f, 0, 1.f);

			if (!bPassesFilters && Settings->bDrawFilterDescriptionForRejectedActors)
			{
				FString ActorString;

				/*for (const UDataSourceFilter* Filter : Ledger.RejectedFilters)
				{
					if (Filter && !Filter->IsA<UDataSourceFilterSet>())
					{
						FText Text;
						Filter->Execute_GetDisplayText(Filter, Text);
						ActorString += Text.ToString();
						ActorString += TEXT("\n");
					}
				}*/

				DrawDebugString(World, FVector::ZeroVector/*Actor->GetActorLocation()*/, ActorString, (AActor*)Actor, FColor::Red, 0.f);
			}
		}
	}
}

void FSourceFilterManager::SetupAsyncTasks(ENamedThreads::Type CurrentThread)
{
	static FGraphEventRef LastApplyAsyncTaskEvent;

	// if the static var left alive it can be destroyed after its allocator
	UE_CALL_ONCE([] { FCoreDelegates::OnEnginePreExit.AddLambda([] { LastApplyAsyncTaskEvent = nullptr; }); });

	if (CAN_TRACE_OBJECT(World))
	{
		if (FilterSetup.HasAnyFilters())
		{
			// Wait for last async task to finish
			WaitForAsyncTasks();

			// Reset frame transient data
			ResetPerFrameData();

			// Setup new tasks for next frame
			FGraphEventArray PrerequistesApply;

			// This ensures that separate worlds (editor) are never trying to apply filter results at the same point in time
			if (LastApplyAsyncTaskEvent != nullptr)
			{
				PrerequistesApply.Add(LastApplyAsyncTaskEvent);
			}

			FGraphEventArray PrerequistesAsync;
			if (FilterSetup.HasGameThreadFilters())
			{
				AsyncTasks[0] = TGraphTask<FActorFilterAsyncTask>::CreateTask(nullptr, CurrentThread).ConstructAndDispatchWhenReady(this, false);
				PrerequistesApply.Add(AsyncTasks[0]);
				PrerequistesAsync.Add(AsyncTasks[0]);
			}

			if (FilterSetup.HasAsyncFilters())
			{
				AsyncTasks[1] = TGraphTask<FActorFilterAsyncTask>::CreateTask(&PrerequistesAsync, CurrentThread).ConstructAndDispatchWhenReady(this, true);
				PrerequistesApply.Add(AsyncTasks[1]);
			}

			FinishTask = LastApplyAsyncTaskEvent = TGraphTask<FActorFilterApplyAsyncTask>::CreateTask(&PrerequistesApply, CurrentThread).ConstructAndDispatchWhenReady(this);
		}

		if (Settings->bDrawFilteringStates)
		{
			FGraphEventArray PrerequistesDraw = { FinishTask };
			DrawTask = TGraphTask<FActorFilterDrawStateAsyncTask>::CreateTask(FinishTask ? &PrerequistesDraw : nullptr, CurrentThread).ConstructAndDispatchWhenReady(this);
		}
	}
}

void FSourceFilterManager::ResetPerFrameData()
{
	QUICK_SCOPE_CYCLE_COUNTER(ResetPerFrameData);

	// Re-init the actor collector with current world state
	ActorCollector.CollectActors();
		
	ResultCache.ResetFrameData(ActorCollector.NumActors(), FilterSetup.NumFilterAndSetEntries());
	
	SetupIntervalData();
}

void FSourceFilterManager::RegisterTickFunctions()
{
	if (!PrePhysicsTickFunction.IsTickFunctionRegistered())
	{
		PrePhysicsTickFunction.TickGroup = ETickingGroup::TG_PrePhysics;
		PrePhysicsTickFunction.Manager = this;
		PrePhysicsTickFunction.RegisterTickFunction(World->PersistentLevel);
	}

	if (!LastDemotableTickFunction.IsTickFunctionRegistered())
	{
		LastDemotableTickFunction.TickGroup = ETickingGroup::TG_LastDemotable;
		LastDemotableTickFunction.Manager = this;
		LastDemotableTickFunction.RegisterTickFunction(World->PersistentLevel);
	}

	bAreTickFunctionsRegistered = true;
}

void FSourceFilterManager::UnregisterTickFunctions()
{
	if (PrePhysicsTickFunction.IsTickFunctionRegistered())
	{
		PrePhysicsTickFunction.UnRegisterTickFunction();
	}	

	if (LastDemotableTickFunction.IsTickFunctionRegistered())
	{
		LastDemotableTickFunction.UnRegisterTickFunction();
	}

	bAreTickFunctionsRegistered = false;
}

void FSourceFilterManager::WaitForAsyncTasks()
{
	check(IsInGameThread());
	if (IsValidRef(FinishTask))
	{
		// Run GT filtering task (in case the GT got pre-empted before executing it, which otherwise causes a deadlock)
		if (IsValidRef(AsyncTasks[0]) && !AsyncTasks[0]->IsComplete())
		{			
			ApplyGameThreadFilters();
			AsyncTasks[0]->DispatchSubsequents(ENamedThreads::GameThread);
		}

		FTaskGraphInterface::Get().WaitUntilTaskCompletes(FinishTask);
	}
}

void FSourceFilterManager::SetupIntervalData()
{
	SetupIntervalFilterSet(FilterSetup.GetRootSet());
}

void FSourceFilterManager::SetupIntervalFilterSet(const FFilterSet& FilterSet)
{
	for (const FFilter& Filter : FilterSet.FilterEntries)
	{
		if (Filter.TickInterval > 1)
		{
			SetupIntervalFilter(Filter);
		}
	}

	for (const FFilterSet& ChildFilterSet : FilterSet.ChildFilterSets)
	{
		SetupIntervalFilterSet(ChildFilterSet);
	}
}

void FSourceFilterManager::SetupIntervalFilter(const FFilter& Filter)
{
	//Filter.TickInterval
	uint32& LastTickFrame = WorldData.IntervalFilterFrames[Filter.TickFrameOffset];
	const uint32 TickDelta = (GFrameNumber - LastTickFrame);
	const bool bShouldTickForThisFrame = (TickDelta >= Filter.TickInterval);
	WorldData.IntervalFilterShouldTick[Filter.TickFrameOffset] = bShouldTickForThisFrame;

	if (bShouldTickForThisFrame)
	{
		LastTickFrame = GFrameNumber;
		ResultCache.AddTickedIntervalFilter(Filter);
	}
}

FSourceFilterManager::FFilterTraceScopeLock::FFilterTraceScopeLock()
{
#if USE_NON_THREADSAFE_TRACE_FILTER_API 
	FTraceFilter::Lock();
#endif
}

FSourceFilterManager::FFilterTraceScopeLock::~FFilterTraceScopeLock()
{
#if USE_NON_THREADSAFE_TRACE_FILTER_API 
	FTraceFilter::Unlock();
#endif
}
