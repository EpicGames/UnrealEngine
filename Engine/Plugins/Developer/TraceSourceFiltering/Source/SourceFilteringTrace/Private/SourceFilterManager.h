// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Async/TaskGraphInterfaces.h"
#include "SourceFilteringTickFunction.h"
#include "ActorFiltering.h"
#include "SourceFilterSetup.h"

class UDataSourceFilter;
class UDataSourceFilterSet;
class UWorld;
class AActor;
class UTraceSourceFilteringSettings;
class USourceFilterCollection;

class FSourceFilterManager;

DECLARE_STATS_GROUP(TEXT("FSourceFilterManager"), STATGROUP_SourceFilterManager, STATCAT_Advanced);

/** Per-UWorld object that keeps track of the its contained AActor's filtering states */
class FSourceFilterManager
{
	friend struct FTraceWorldFiltering;
	friend struct FSourceFilteringTickFunction;
	
	friend class FTraceSourceFiltering;
	friend class FActorFilterAsyncTask;
	friend class FActorFilterApplyAsyncTask;
	friend class FActorFilterDrawStateAsyncTask;

	friend class FTraceSourceFilteringTestBase;
public:
	FSourceFilterManager(UWorld* InWorld);
	~FSourceFilterManager();
	
protected:	
	/** Update the internal data based upon the changed filter setup */
	void OnFilterSetupChanged();
	void OnSourceFilteringSettingsChanged();
	void ResetFilterData();

	/** Applies all UDataSourceFilters to the specified AActor, and updates it filtering state accordingly */
	template<EFilterType FilterType>
	void ApplyFilters();

	void ApplyGameThreadFilters();
	void ApplyAsyncFilters();

	template<EFilterType FilterType>
	bool ApplyFilterSetToActor(const FFilterSet& FilterSet, const FActorFilterInfo& ActorInfo);
		
	// Spawn filtering
	void ApplySpawnFilters();
	void ApplySpawnFiltersToActor(const AActor* Actor);
	void AddOnSpawnFilter(const UDataSourceFilter* Filter, bool bExpectedResult, uint32 RunHash);
			
	/** Apply the previously determined results to the Trace filtering state */
	void ApplyFilterResults();	

	/** Determine the filtering state for an individual Actor */
	void ApplyFilterResultsToActor(const FActorFilterInfo& ActorInfo);	
	bool DetermineActorFilteringState(const FActorFilterInfo& ActorInfo) const;

	bool ApplyClassFilterToActor(const AActor* Actor);
	
	// Debug drawing
	void DrawFilterResults();
	void DrawFilterResults(const AActor* Actor);
	
	// Set and kick-off async tasks
	void SetupAsyncTasks(ENamedThreads::Type CurrentThread);

	/** Waits for any previously kicked of async tasks */
	void WaitForAsyncTasks();

	/** Reset last frame's transient data and setup the next */
	void ResetPerFrameData();

	/** Setting up per-frame data for any contained interval-based filters */
	void SetupIntervalData();
	void SetupIntervalFilterSet(const FFilterSet& FilterSet);
	void SetupIntervalFilter(const FFilter& Filter);

	/** Tick functions used to kick-off the individual (async) tasks */
	void RegisterTickFunctions();
	void UnregisterTickFunctions();	

protected:
	/** Registered delegate for whenever an AActor is spawned, and before, within World */
	FDelegateHandle ActorSpawningDelegateHandle;
	FDelegateHandle PreActorSpawningDelegateHandle;

	/** Filtering settings for the running instance */
	UTraceSourceFilteringSettings* Settings;	

	/** UWorld instance that this instance corresponds to */
	UWorld* World;
	
	/** Maps from Actor Hash to FilterSets (hash) that will pass due to previous Spawn Only filter results */
	TMap<uint32, FHashSet> PassedFilterSets;
	/** Maps from Actor Hash to FilterSets (hash) that will not pass due to previous Spawn Only filter results */
	TMap<uint32, FHashSet> DiscardedFilterSets;
		
	/** Actor hashes for those that were previously discarded by, or passed a spawn filter*/
	TSet<uint32> DiscardedActorHashes;
	TSet<uint32> PassedActorsHashes;
	
	/** Tick functions */
	FSourceFilteringTickFunction PrePhysicsTickFunction;
	FSourceFilteringTickFunction LastDemotableTickFunction;	
	
	/** Async task-graph handles */
	FGraphEventRef AsyncTasks[2];
	FGraphEventRef FinishTask;
	FGraphEventRef DrawTask;
	
	/** Flags used to determine what type (or if any) filters are contained in the current filter-set */	
	uint8 bAreTickFunctionsRegistered : 1;
	
	struct FFilterStats
	{
		uint32 TotalActors;
		uint32 EarlyRejectedActors;
		uint32 EarlyPassedActors;
		uint32 RejectedActors;
		uint32 PassedActors;
		uint32 EvaluatedActors;
	};
	FFilterStats Statistics;
		
	FResultCache ResultCache;

	TArray<uint32> ActorHashes;	
	FFilteredActorCollector ActorCollector;

	FSourceFilterSetup& FilterSetup;
	FPerWorldData WorldData;

	struct FFilterTraceScopeLock
	{
		FFilterTraceScopeLock();
		~FFilterTraceScopeLock();
	};
};
