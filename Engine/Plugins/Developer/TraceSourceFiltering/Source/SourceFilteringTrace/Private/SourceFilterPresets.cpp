// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceFilterPresets.h"

#include "AssetRegistry/AssetData.h"
#include "HAL/IConsoleManager.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Blueprint/BlueprintSupport.h"

#include "Modules/ModuleManager.h"
#include "TraceSourceFiltering.h"
#include "SourceFilterCollection.h"
#include "SourceFilterTrace.h"

#if SOURCE_FILTER_TRACE_ENABLED
static FAutoConsoleCommand GListAvailablePresetsCommand(
	TEXT("Trace.ListPresets"),
	TEXT("Lists all the available filtering presets"),
	FConsoleCommandDelegate::CreateStatic(&FSourceFilterPresets::ListAvailablePresets)
);

static FAutoConsoleCommand GLoadPresetCommand(
	TEXT("Trace.LoadPreset"),
	TEXT("Loads a specific filtering preset\n")
	TEXT("Argument:\n")
	TEXT("Integer: index of preset within Trace.ListPresets output\n")
	TEXT("or\n")
	TEXT("String: path to UTraceFilterCollection object\n"),
	FConsoleCommandWithArgsDelegate::CreateStatic(&FSourceFilterPresets::LoadPresetCommand)
);
#endif // SOURCE_FILTER_TRACE_ENABLED

DEFINE_LOG_CATEGORY_STATIC(SourceFilterPresets, Display, Display);

void FSourceFilterPresets::ListAvailablePresets()
{
	TArray<FAssetData> PresetAssetData;
	GetPresets(PresetAssetData);

	for (int32 AssetDataIndex = 0; AssetDataIndex < PresetAssetData.Num(); ++AssetDataIndex)
	{
		const FAssetData& AssetData = PresetAssetData[AssetDataIndex];
		UE_LOGF(SourceFilterPresets, Display, "[%i]: %ls\n", AssetDataIndex, *AssetData.GetExportTextName());
	}
}

void FSourceFilterPresets::GetPresets(TArray<FAssetData>& InOutPresetAssetData)
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	TMultiMap<FName, FString> TagValues = { { FBlueprintTags::NativeParentClassPath, FObjectPropertyBase::GetExportPath(USourceFilterCollection::StaticClass()) } };
	AssetRegistryModule.Get().GetAssetsByClass(USourceFilterCollection::StaticClass()->GetClassPathName(), InOutPresetAssetData);
}

void FSourceFilterPresets::LoadPreset(const FSoftObjectPath& PresetPath)
{
	if (USourceFilterCollection* Preset = Cast<USourceFilterCollection>(PresetPath.TryLoad()))
	{
		FTraceSourceFiltering::Get().GetFilterCollection()->CopyData(Preset);

		UE_LOGF(SourceFilterPresets, Display, "Loaded Trace Source Filter preset: %ls\n", *PresetPath.ToString());
	}
	else
	{
		// Unable to load
		UE_LOGF(SourceFilterPresets, Display, "Failed to find preset: %ls\n", *PresetPath.ToString());
	}
}

void FSourceFilterPresets::LoadPresetCommand(const TArray<FString>& Arguments)
{
	// Only expecting a single argument
	if (Arguments.Num() == 1)
	{
		int32 Index = 0;
		if (LexTryParseString(Index, *Arguments[0]))
		{
			TArray<FAssetData> PresetAssetData;
			GetPresets(PresetAssetData);

			// Retrieve specific asset data at user index
			if (PresetAssetData.IsValidIndex(Index))
			{
				LoadPreset(PresetAssetData[Index].GetObjectPathString());
			}
			else
			{
				UE_LOGF(SourceFilterPresets, Display, "Invalid preset index\n");
				ListAvailablePresets();
			}
		}
		else
		{
			FSoftObjectPath Path(Arguments[0]);
			if (Path.IsValid())
			{
				LoadPreset(Path);
			}
		}
	}
	else
	{
		UE_LOGF(SourceFilterPresets, Display, "Expecting a single argument:\n"
			"Integer: index of preset within Trace.ListPresets output\n"
			"or\n"
			"String: path to UTraceFilterCollection object\n");
	}
}
