// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"

struct FAssetData;
struct FSoftObjectPath;
class FString;

struct FSourceFilterPresets
{	
	/** Return array of FAssetData entries representing load-able filter presets */
	static void GetPresets(TArray<FAssetData>& InOutPresetAssetData);

	/** Loading and replacing the current filtering state with a preset */
	static void LoadPreset(const FSoftObjectPath& PresetPath);
	
	/** Logs all available filter preset paths */
	static void ListAvailablePresets();
	/** Console command functionality for loading a preset my path or index (relative to GetPresets())*/
	static void LoadPresetCommand(const TArray<FString>& Arguments);
};