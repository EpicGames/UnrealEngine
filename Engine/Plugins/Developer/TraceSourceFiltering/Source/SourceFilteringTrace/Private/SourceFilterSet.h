// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "SourceFilter.h" // IWYU pragma: keep

enum class EFilterSetMode : uint8;

typedef TInlineAllocator<8> FilterSetAllocator;

/** Simplified structure representing a DataSourceFilterSet and its state within the FSourceFilterManager */
struct FFilterSet
{
	/** Contained (at least one) filters */
	TArray<FFilter, FilterSetAllocator> FilterEntries;

	/** Any contained child filter sets */
	TArray<FFilterSet> ChildFilterSets;

	/** Typehash for the corresponding UDataSourceFilterSet */
	uint32 FilterSetHash;

	/** Cached Offset into the result (bit) array */
	uint32 ResultOffset;

	/** Set operation */
	EFilterSetMode Mode;

	/** Whether or not this filter set contains any gamethread filters */
	uint8 bContainsGameThreadFilter : 1;
	/** Whether or not this filter set contains any async filters */
	uint8 bContainsAsyncFilter : 1;
	/** Expected value (true/false) for whether or not an applied filter within this set passes */
	uint8 bInitialPassingValue : 1;	
};
