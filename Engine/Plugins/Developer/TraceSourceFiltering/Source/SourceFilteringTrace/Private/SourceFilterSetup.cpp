// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceFilterSetup.h"

#include "Algo/Accumulate.h"
#include "Misc/CoreDelegates.h"


#include "DataSourceFilterSet.h"
#include "TraceSourceFiltering.h"
#include "TraceSourceFilteringSettings.h"
#include "SourceFilterCollection.h"

#include "Misc/CoreDelegates.h"

DEFINE_LOG_CATEGORY_STATIC(SourceFilterSetup, Display, Display);

FSourceFilterSetup& FSourceFilterSetup::GetFilterSetup()
{
	static FSourceFilterSetup FilterSetup;
	return FilterSetup;
}

void FSourceFilterSetup::OnSourceFiltersUpdated()
{
	// Reset all resident filtering data
	Reset();
	
	// Retrieve current set of filters
	const TArray<UDataSourceFilter*>& TopLevelFilters = FilterCollection->GetFilters();

	if (TopLevelFilters.Num())
	{
		SetupUserFilters(TopLevelFilters, FilterSettings->bOutputOptimizedFilterState);
	}

	ActorClassFilters = FilterCollection->GetClassFilters();

	FilterSetupUpdated.Broadcast();
}


FSourceFilterSetup::FSourceFilterSetup()
{
	FilterCollection = FTraceSourceFiltering::Get().GetFilterCollection();
	FilterSettings = FTraceSourceFiltering::Get().GetSettings();

	OnSourceFiltersUpdated();
	
	FilterCollection->GetSourceFiltersUpdated().AddRaw(this, &FSourceFilterSetup::OnSourceFiltersUpdated);

	FCoreDelegates::OnPreExit.AddRaw(this, &FSourceFilterSetup::ShutdownOnPreExit);
}

FSourceFilterSetup::~FSourceFilterSetup()
{
	FCoreDelegates::OnPreExit.RemoveAll(this);
}

void FSourceFilterSetup::ShutdownOnPreExit()
{
	FilterCollection->GetSourceFiltersUpdated().RemoveAll(this);
}

void FSourceFilterSetup::Reset()
{
	bContainsGameThreadFilters = false;
	bContainsNonGameThreadFilters = false;
	bContainsAnyFilters = false;
	bContainsAnySpawnOnlyFilters = false;

	OnSpawnFilterSetDiscards.Empty();
	OnSpawnFilterSetPass.Empty();

	OnSpawnFilters.Empty();

	TotalNumEntries = 0;
	TotalIntervalEntries = 0;
}

void FSourceFilterSetup::SetupUserFilters(const TArray<UDataSourceFilter*>& Filters, bool bOutputState)
{
	// Setup root filter set
	RootSet = FFilterSet();
	// Root-level contains OR-ed entries
	RootSet.Mode = EFilterSetMode::OR;
	RootSet.bInitialPassingValue = false;
	// Root level hash value (0)
	RootSet.FilterSetHash = 0;

	RootSet.ResultOffset = TotalNumEntries++;

	/** Root level expected resulting value = true */
	bool bResultValue = true;

	/** Recursively iterate through filters */
	for (const UDataSourceFilter* Filter : Filters)
	{
		ProcessFilterRecursively(Filter, RootSet, bResultValue);
	}

	if (bOutputState)
	{
		FString OutputString;
		OutputFilterSetState(RootSet, OutputString);
		UE_LOGF(SourceFilterSetup, Display, "Before optimization:\n%ls", *OutputString);
	}

	OptimizeFiltersInSet(RootSet);

	if (bOutputState)
	{
		FString OutputString;
		OutputFilterSetState(RootSet, OutputString);
		UE_LOGF(SourceFilterSetup, Display, "After optimization:\n%ls", *OutputString);
	}

	if (bContainsAnySpawnOnlyFilters)
	{
		// Process added spawn filter entries, storing hash for each filter set, that would either pass or be rejected when the spawn-only filter does
		for (const TPair<uint32, FHashSet>& RejectFilterSetHashes : OnSpawnFilterSetDiscards)
		{
			if (RejectFilterSetHashes.Value.Contains(0))
			{
				// In case this is the only spawn and top-level filter, it can early out (false) 
				if (OnSpawnFilters.Num() == 1 && !bContainsGameThreadFilters && !bContainsNonGameThreadFilters)
				{
					GetSpawnFilter(RejectFilterSetHashes.Key).bEarlyOutDiscard = 1;
				}
			}
		}

		for (const TPair<uint32, FHashSet>& PassFilterSetHashes : OnSpawnFilterSetPass)
		{
			if (PassFilterSetHashes.Value.Contains(0))
			{
				if (OnSpawnFilters.Num() == 1 && !bContainsGameThreadFilters && !bContainsNonGameThreadFilters)
				{
					GetSpawnFilter(PassFilterSetHashes.Key).bEarlyOutPass = 1;
				}
			}
		}
	}
}

void FSourceFilterSetup::ProcessFilterRecursively(const UDataSourceFilter* InFilter, FFilterSet& InOutFilterSet, bool bResultValue)
{
	// Only need to process if enabled
	if (InFilter->IsEnabled())
	{
		const bool bParentSetHasANDOperation = InOutFilterSet.Mode == EFilterSetMode::AND;

		// Process filter set
		if (const UDataSourceFilterSet* FilterSet = Cast<UDataSourceFilterSet>(InFilter))
		{
			// Cache the result value
			bool bSetResultValue = bResultValue;

			// NOT operation, means we'll expect the inverted result value of applying the filters
			if (FilterSet->GetFilterSetMode() == EFilterSetMode::NOT)
			{
				bSetResultValue = !bSetResultValue;
			}

			const EFilterSetMode FilterSetOperation = FilterSet->GetFilterSetMode();
			const bool bSetHasANDOperation = FilterSetOperation == EFilterSetMode::AND;

			// Operations between this set and the parent match, meaning we can inline our filters into the parent
			const bool bMatchingSetOperations = bParentSetHasANDOperation == bSetHasANDOperation;
			
			const uint32 NumEnabledFilters = Algo::Accumulate(FilterSet->Filters, 0, [](uint32 Value, auto& Filter)
			{
				return (Value + !!Filter->IsEnabled());
			});
			
			// Filter set only contains a single entry, or operations match, means we can inline it into the parent
			const bool bSingleContainedFilterSet = NumEnabledFilters == 1;			
			if (bMatchingSetOperations || bSingleContainedFilterSet)
			{
				for (const UDataSourceFilter* ChildFilter : FilterSet->Filters)
				{
					ProcessFilterRecursively(ChildFilter, InOutFilterSet, bSetResultValue);
				}
			}
			else
			{
				// Insert a child set entry into the parent set
				FFilterSet& ChildSet = InOutFilterSet.ChildFilterSets.AddDefaulted_GetRef();
				ChildSet.Mode = FilterSetOperation;
				ChildSet.FilterSetHash = GetTypeHash(FilterSet);
				ChildSet.bInitialPassingValue = DetermineDefaultPassingValueForSet(FilterSet);

				ChildSet.ResultOffset = TotalNumEntries++;

				// Add all filters to the new child set
				for (const UDataSourceFilter* ChildFilter : FilterSet->Filters)
				{
					ProcessFilterRecursively(ChildFilter, ChildSet, bSetResultValue);
				}

				InOutFilterSet.bContainsAsyncFilter |= ChildSet.bContainsAsyncFilter;
				InOutFilterSet.bContainsGameThreadFilter |= ChildSet.bContainsGameThreadFilter;
			}
		}
		else
		{
			const bool bSpawnOnlyFilter = InFilter->Configuration.bOnlyApplyDuringActorSpawn;
			const bool bGameThreadFilter = !InFilter->Configuration.bCanRunAsynchronously;

			// Collating various flags
			bContainsGameThreadFilters |= (bGameThreadFilter && !bSpawnOnlyFilter);
			bContainsNonGameThreadFilters |= (!bGameThreadFilter && !bSpawnOnlyFilter);
			bContainsAnySpawnOnlyFilters |= bSpawnOnlyFilter;
			bContainsAnyFilters = true;

			AddFilterToSet(InOutFilterSet, InFilter, bResultValue);
			if (bSpawnOnlyFilter)
			{
				OnSpawnFilters.Add(InOutFilterSet.FilterEntries.Last());
			}
		}
	}
}

bool FSourceFilterSetup::DetermineDefaultPassingValueForSet(const UDataSourceFilterSet* FilterSet)
{
	if (FilterSet->Filters.Num() == 0)
	{
		return true;
	}

	switch (FilterSet->Mode)
	{
	case EFilterSetMode::OR:
		return false;
	case EFilterSetMode::AND:
		return true;
	case EFilterSetMode::NOT:
		return true;
	default:
		return false;
	}
}

void FSourceFilterSetup::OutputFilterSetState(const FFilterSet& FilterSet, FString& OutputString, int32 Depth)
{
	const UEnum* FilterModeEnum = FindObject<UEnum>(nullptr, TEXT("/Script/SourceFilteringCore.EFilterSetMode"));

	FString NewLineValue = TEXT("\n");
	FString TabValue = TEXT("\t");
	for (int32 i = 0; i < Depth; ++i)
	{
		NewLineValue += TEXT("\t");
		TabValue += TEXT("\t");
	}

	OutputString += NewLineValue;

	OutputString += TEXT("Filter Set - ");
	OutputString += FilterModeEnum->GetNameStringByValue((int64)FilterSet.Mode);
	OutputString += NewLineValue;

	for (int32 FilterIndex = 0; FilterIndex < FilterSet.FilterEntries.Num(); ++FilterIndex)
	{
		const FFilter& FilterEntry = FilterSet.FilterEntries[FilterIndex];
		const UDataSourceFilter* Filter = FilterEntry.Filter;

		OutputString += TabValue;
		FText Display;
		Filter->Execute_GetDisplayText(Filter, Display);
		if (!FilterEntry.bExpectedValue)
			OutputString += TEXT("!");

		OutputString += Display.ToString();
		OutputString += NewLineValue;
	}

	for (const FFilterSet& ChildFilterSet : FilterSet.ChildFilterSets)
	{
		OutputFilterSetState(ChildFilterSet, OutputString, Depth + 1);
	}

	OutputString += NewLineValue;
}

FFilter& FSourceFilterSetup::GetSpawnFilter(uint32 FilterHash)
{
	FFilter* Filter = OnSpawnFilters.FindByPredicate([FilterHash](FFilter& Entry)
	{
		return Entry.FilterHash == FilterHash;
	});
	check(Filter);

	return *Filter;
}

FSourceFilterSetup::FOptimizationResult FSourceFilterSetup::OptimizeFiltersInSet(FFilterSet& FilterSet)
{
	/** The goal here is to minimize the filters we have to process for any given actor, providing early outs and ordering them according to their
		approximated costs. */
	FOptimizationResult Result;

	int32 TotalCost = 0;

	struct FSortStruct
	{
		int32 Index;
		FOptimizationResult Result;
	};
	TArray<FSortStruct> ChildFiltersCosts;

	auto GetApproximatedFilterCosts = [](const FFilter& Filter) -> int32
	{
		return Filter.bOnSpawnOnly ? 0 : (Filter.bNative ? 1 : 8);
	};

	// Sort each filter entry according to their approximated costs
	FilterSet.FilterEntries.Sort([GetApproximatedFilterCosts](const FFilter& FilterA, const FFilter& FilterB)
	{
		return GetApproximatedFilterCosts(FilterA) < GetApproximatedFilterCosts(FilterB);
	});

	// Calculate total cost for this set
	for (const FFilter& Entry : FilterSet.FilterEntries)
	{
		TotalCost += GetApproximatedFilterCosts(Entry);
	}

	// Apply the same optimizations to all child filter sets 
	ChildFiltersCosts.SetNumZeroed(FilterSet.ChildFilterSets.Num());
	for (int32 ChildFilterSetIndex = 0; ChildFilterSetIndex < FilterSet.ChildFilterSets.Num(); ++ChildFilterSetIndex)
	{
		FFilterSet& ChildSet = FilterSet.ChildFilterSets[ChildFilterSetIndex];
		ChildFiltersCosts[ChildFilterSetIndex].Result = OptimizeFiltersInSet(ChildSet);
		ChildFiltersCosts[ChildFilterSetIndex].Index = ChildFilterSetIndex;
	}

	// Sort child filter sets according to their total costs
	ChildFiltersCosts.Sort([](const FSortStruct& SetA, const FSortStruct& SetB)
	{
		return SetA.Result.TotalCost < SetB.Result.TotalCost;
	});

	// Generate new child filter set array according the previous sorting result
	TArray<FFilterSet> SortedChildSets;
	for (int32 ChildFilterSetIndex = 0; ChildFilterSetIndex < FilterSet.ChildFilterSets.Num(); ++ChildFilterSetIndex)
	{
		SortedChildSets.Add(FilterSet.ChildFilterSets[ChildFiltersCosts[ChildFilterSetIndex].Index]);
	}

	// Store the sorted sets
	FilterSet.ChildFilterSets = SortedChildSets;

	// Accumulate the number of Spawn Only filters
	const int32 NumSpawnFilters = Algo::Accumulate(FilterSet.FilterEntries, 0, [](int32 Value, const FFilter& Entry)
	{
		Value += Entry.bOnSpawnOnly ? 1 : 0;
		return Value;
	});

	const bool bFilterSetHasANDOperation = FilterSet.Mode == EFilterSetMode::AND;

	// In case this set contains any spawn filters
	if (NumSpawnFilters)
	{
		/** A single filter set can be early-ed out (true or false) as the single filter will determine the set outcome */
		const bool bSingleFilterSet = FilterSet.FilterEntries.Num() == 1;

		/** We can early out (true) of a set if its operation is either OR or NOT, as one passing filter is required for the set to pass */
		const bool bCanEarlyOutOnPass = !bFilterSetHasANDOperation || bSingleFilterSet;

		/** We can early out (false) of a set if its operation is AND, as one passing filter failing will make the set fail */
		const bool bCanDiscardOnFail = bFilterSetHasANDOperation || bSingleFilterSet;

		if (bCanEarlyOutOnPass || bCanDiscardOnFail)
		{
			for (const FFilter& Entry : FilterSet.FilterEntries)
			{
				if (Entry.bOnSpawnOnly)
				{
					if (bCanEarlyOutOnPass)
					{
						OnSpawnFilterSetPass.FindOrAdd(Entry.FilterHash).Add(FilterSet.FilterSetHash);
					}

					if (bCanDiscardOnFail)
					{
						OnSpawnFilterSetDiscards.FindOrAdd(Entry.FilterHash).Add(FilterSet.FilterSetHash);
					}

					// Forward the filter hash to parent set
					Result.OnSpawnFilterHashes.Add(Entry.FilterHash);
				}
			}
		}
	}

	// Process child sets, finding out whether or not the result of a child filter set can cause the outer set to early out
	for (int32 ChildFilterSetIndex = 0; ChildFilterSetIndex < FilterSet.ChildFilterSets.Num(); ++ChildFilterSetIndex)
	{
		/** We can early out (true) in case the parent set operation is OR/NOT, or in case this is the only child set */
		const bool bCanEarlyOutSetOnPass = (!bFilterSetHasANDOperation || (FilterSet.ChildFilterSets.Num() == 1 && FilterSet.FilterEntries.Num() == 0));

		/** We can early out (false) in case the parent set operation is AND and contains multiple child sets, or in case this is the only child set, and the parent does not contain any filters of its own */
		const bool bCanDiscardSetOnFail = ((bFilterSetHasANDOperation && FilterSet.ChildFilterSets.Num() > 1) || (FilterSet.ChildFilterSets.Num() == 1 && FilterSet.FilterEntries.Num() == 0));

		const uint32 ChildSetHash = FilterSet.ChildFilterSets[ChildFilterSetIndex].FilterSetHash;

		const FOptimizationResult& OptimizeResult = ChildFiltersCosts[ChildFilterSetIndex].Result;
		for (const uint32& ChildEntryHash : OptimizeResult.OnSpawnFilterHashes)
		{
			// Ensure that child-sets were optimized and marked to be skipped for this given spawn filter
			const bool bChildSetCanBePassed = OnSpawnFilterSetPass.Contains(ChildEntryHash) && OnSpawnFilterSetPass.FindChecked(ChildEntryHash).Contains(ChildSetHash);
			if (bCanEarlyOutSetOnPass && bChildSetCanBePassed)
			{
				OnSpawnFilterSetPass.FindOrAdd(ChildEntryHash).Add(FilterSet.FilterSetHash);
				Result.OnSpawnFilterHashes.Add(ChildEntryHash);
			}

			const bool bChildSetCanEarlyOut = (!bFilterSetHasANDOperation || (OnSpawnFilterSetDiscards.Contains(ChildEntryHash) && OnSpawnFilterSetDiscards.FindChecked(ChildEntryHash).Contains(ChildSetHash)));
			if (bCanDiscardSetOnFail && bChildSetCanEarlyOut)
			{
				OnSpawnFilterSetDiscards.FindOrAdd(ChildEntryHash).Add(FilterSet.FilterSetHash);
				Result.OnSpawnFilterHashes.Add(ChildEntryHash);
			}
		}
	}

	Result.NumSpawnFilters = NumSpawnFilters;
	Result.TotalCost = TotalCost;
	return Result;
}

void FSourceFilterSetup::PopulatePerWorldData(FPerWorldData& InData)
{
	InData.IntervalFilterFrames.SetNumZeroed(TotalIntervalEntries);
	InData.IntervalFilterShouldTick.SetNumZeroed(TotalIntervalEntries);
}

void FSourceFilterSetup::AddFilterToSet(FFilterSet& FilterSet, const UDataSourceFilter* Filter, bool bExpectedResult)
{
	FFilter& Entry = FilterSet.FilterEntries.AddDefaulted_GetRef();

	Entry.Filter = Filter;
	Entry.FilterHash = GetTypeHash(Filter);
	Entry.bExpectedValue = bExpectedResult;
	Entry.bNative = Filter->GetClass()->HasAnyClassFlags(CLASS_Native);
	Entry.bOnSpawnOnly = Filter->Configuration.bOnlyApplyDuringActorSpawn;
	Entry.bCanRunAsynchronously = Filter->Configuration.bCanRunAsynchronously;
	Entry.TickInterval = Filter->Configuration.FilterApplyingTickInterval;
	Entry.FilterSetHash = FilterSet.FilterSetHash;

	// In case the set operation is OR / NOT, we can early out (true) if the filter passes
	Entry.bEarlyOutPass = FilterSet.Mode == EFilterSetMode::OR || FilterSet.Mode == EFilterSetMode::NOT;
	// In case the set operation is AND, we can early out (false) if the filter does not pass
	Entry.bEarlyOutDiscard = FilterSet.Mode == EFilterSetMode::AND;

	// Filter set flags
	FilterSet.bContainsAsyncFilter |= Entry.bCanRunAsynchronously;
	FilterSet.bContainsGameThreadFilter |= !Entry.bCanRunAsynchronously;

	Entry.ResultOffset = TotalNumEntries++;

	if (Entry.TickInterval > 1)
	{
		Entry.TickFrameOffset = TotalIntervalEntries++;
	}
}
