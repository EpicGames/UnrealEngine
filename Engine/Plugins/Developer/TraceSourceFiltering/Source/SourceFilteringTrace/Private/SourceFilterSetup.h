// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "ResultCache.h"

#include "SourceFilterSet.h"

struct FActorClassFilter;

struct FFilter;
struct FFilterSet;
class UDataSourceFilter;
class UDataSourceFilterSet;

class UTraceSourceFilteringSettings;
class USourceFilterCollection;

struct FPerWorldData
{
	/** Cache containing frame indices at which each interval filter was last evaluated */
	TArray<uint32, FilterSetAllocator> IntervalFilterFrames;
	/** Contains flag for each interval filter, determining whether or not it should be evaluated this frame */
	TArray<bool, FilterSetAllocator> IntervalFilterShouldTick;
};

class FSourceFilterSetup
{
public:
	/** Get singleton filter setup instance */
	static FSourceFilterSetup& GetFilterSetup();
	
	/** Retrieve delegate broadcasted whenever the contained setup changes */
	FSimpleMulticastDelegate& GetFilterSetupUpdated() { return FilterSetupUpdated; }

	/** Populates per-world data based upon the current setup */
	void PopulatePerWorldData(FPerWorldData& InData);

	/** Return (top-level) root filter set */
	const FFilterSet& GetRootSet() const { return RootSet; }
	/** Retrieve setup spawn filters */
	const TArray<FFilter>& GetSpawnFilters() const { return OnSpawnFilters; }
	/** Retriever setup actor-class filters */
	const TArray<FActorClassFilter>& GetActorFilters() const { return ActorClassFilters; }

	/** Retrieve array of filter-set hashes which can be regarded as rejected for the provided spawn filter */
	FHashSet GetSpawnFilterDiscardableSets(const FFilter& SpawnFilter) 
	{
		const FHashSet* DiscardHashes = OnSpawnFilterSetDiscards.Find(SpawnFilter.FilterHash);		
		return DiscardHashes ? *DiscardHashes : FHashSet();
	}

	/** Retrieve array of filter-set hashes which can be regarded as passing for the provided spawn filter */
	FHashSet GetSpawnFilterSkippableSets(const FFilter& SpawnFilter)
	{
		const FHashSet* DiscardHashes = OnSpawnFilterSetPass.Find(SpawnFilter.FilterHash);
		return DiscardHashes ? *DiscardHashes : FHashSet();
	}

	/** Whether nor not the setup contains any filters that need per-frame evaluation / applying */
	bool RequiresApplyingFilters() const { return bContainsGameThreadFilters || bContainsNonGameThreadFilters; }	
	/** Whether or not the setup contains any filters at all */
	bool HasAnyFilters() const { return bContainsAnyFilters; }
	/** Whether or not the setup contains any spawn filters */
	bool HasAnySpawnFilters() const { return bContainsAnySpawnOnlyFilters; }
	/** Whether or not the setup contains only spawn filters  */
	bool HasOnlySpawnFilters() const { return bContainsAnySpawnOnlyFilters && !bContainsGameThreadFilters && !bContainsNonGameThreadFilters; }
	/** Whether or not the setup contains any filters requiring evaluation on the gamethread */
	bool HasGameThreadFilters() const { return bContainsGameThreadFilters; }
	/** Whether or not the setup contains any filters can be evaluated off the gamethread */
	bool HasAsyncFilters() const { return bContainsNonGameThreadFilters; }
	/** Whether or not the setup contains any actor class filters */
	bool HasAnyClassFilters() const { return !!ActorClassFilters.Num(); }

	/** Total number of filter(set)s contained by the setup */
	uint32 NumFilterAndSetEntries() const { return TotalNumEntries; }
protected:
	FSourceFilterSetup();
	~FSourceFilterSetup();

	/** Resets any contained data */
	void Reset();

	/** FCoreDelegates::OnPreExit shutdown callback */
	void ShutdownOnPreExit();

	/** Callback registered with SourceFilterCollection */
	void OnSourceFiltersUpdated();

	/** Generated internal data according to provided filters */
	void SetupUserFilters(const TArray<UDataSourceFilter*>& Filters, bool bOutputState);
	void ProcessFilterRecursively(const UDataSourceFilter* InFilter, FFilterSet& InOutFilterSet, bool bResultValue);
	void AddFilterToSet(FFilterSet& FilterSet, const UDataSourceFilter* Filter, bool bExpectedResult);

	struct FOptimizationResult
	{
		int32 TotalCost;
		int32 NumSpawnFilters;
		TSet<uint32> OnSpawnFilterHashes;
	};
	/** Used to optimize filtering setup, flattening filter sets where possible and reordering filter evaluation according to their estimated cost */
	FOptimizationResult OptimizeFiltersInSet(FFilterSet& FilterSet);

	/** Find spawn-only filter according to its hash */
	FFilter& GetSpawnFilter(uint32 FilterHash);
	
	/** Determine what the expected boolean return value is when evaluating the filter set */
	bool DetermineDefaultPassingValueForSet(const UDataSourceFilterSet* FilterSet);

	/** Recursively outputs the filtering state for the provided filter set*/
	void OutputFilterSetState(const FFilterSet& FilterSet, FString& Output, int32 Depth = 0);	
protected:
	/** Root filter set, contains TopLevelFilters entries */
	FFilterSet RootSet;

	/** All Spawn only filters contained by Rootset*/
	TArray<FFilter> OnSpawnFilters;

	/** Actor class filter */
	TArray<FActorClassFilter> ActorClassFilters;

	/** Maps from individual FFilterEntry Hash (Spawn filters only), to an array of FFilterRun hashes
		that can be disregarded when applying them to an Actor, as they will fail */
	TMap<uint32, FHashSet> OnSpawnFilterSetDiscards;

	/** Maps from individual FFilterEntry Hash (Spawn filters only), to an array of FFilterRun hashes
	that can be disregarded when applying them to an Actor, as they will pass */
	TMap<uint32, FHashSet> OnSpawnFilterSetPass;
	
	/** Filtering settings for the running instance */
	UTraceSourceFilteringSettings* FilterSettings;
	/** Filter collection, containing the UDataSourceFilters, for the running instance */
	USourceFilterCollection* FilterCollection;

	FSimpleMulticastDelegate FilterSetupUpdated;

	/** Flags used to determine what type (or if any) filters are contained in the current filter-set */
	uint8 bContainsGameThreadFilters : 1;
	uint8 bContainsNonGameThreadFilters : 1;
	uint8 bContainsAnyFilters : 1;
	uint8 bContainsAnySpawnOnlyFilters : 1;

	uint32 TotalNumEntries;
	uint32 TotalIntervalEntries;
};
