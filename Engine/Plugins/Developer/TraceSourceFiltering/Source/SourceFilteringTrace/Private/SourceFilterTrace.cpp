// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceFilterTrace.h"
#include "Trace/Detail/Field.h"

#if SOURCE_FILTER_TRACE_ENABLED

#include "DataSourceFilterSet.h"
#include "Trace/Trace.inl"

#include "ObjectTrace.h"
#include "TraceWorldFiltering.h"
#include "EmptySourceFilter.h"

#if WITH_EDITOR
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "SourceFilterTrace"

UE_TRACE_CHANNEL_DEFINE(TraceSourceFiltersChannel)

UE_TRACE_EVENT_BEGIN(SourceFilters, FilterClass)
	UE_TRACE_EVENT_FIELD(uint64, ClassId)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(SourceFilters, FilterInstance)
	UE_TRACE_EVENT_FIELD(uint64, InstanceId)
	UE_TRACE_EVENT_FIELD(uint64, ClassId)
	UE_TRACE_EVENT_FIELD(uint64, SetId)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, DisplayString)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(SourceFilters, FilterSet)
	UE_TRACE_EVENT_FIELD(uint64, SetId)
	UE_TRACE_EVENT_FIELD(uint8, Mode)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(SourceFilters, FilterOperation)
	UE_TRACE_EVENT_FIELD(uint64, InstanceId)
	UE_TRACE_EVENT_FIELD(uint8, Operation)	
	UE_TRACE_EVENT_FIELD(uint64, Parameter)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(SourceFilters, SetFilterSettingValue)
	UE_TRACE_EVENT_FIELD(uint8, Value)
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, PropertyName)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(WorldSourceFilters, WorldInstance)
	UE_TRACE_EVENT_FIELD(uint64, InstanceId)
	UE_TRACE_EVENT_FIELD(uint8, Type)
	UE_TRACE_EVENT_FIELD(uint8, State)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, SendName)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(WorldSourceFilters, WorldOperation)
	UE_TRACE_EVENT_FIELD(uint64, InstanceId)
	UE_TRACE_EVENT_FIELD(uint8, Operation)
	UE_TRACE_EVENT_FIELD(uint32, Parameter)
UE_TRACE_EVENT_END()

TMap<FObjectKey, uint64> FSourceFilterTrace::FilterClassIds;
TMap<uint64, FObjectKey> FSourceFilterTrace::IDToFilterClass;
TMap<uint64, FObjectKey> FSourceFilterTrace::IDToFilter;
TSet<FObjectKey> FSourceFilterTrace::FilterInstances;
TMap<FString, FObjectKey> FSourceFilterTrace::DataSourceFilterClasses;

TMap<uint64, FObjectKey> FSourceFilterTrace::IDsToWorldInstance;

void FSourceFilterTrace::OutputClass(const TSubclassOf<UDataSourceFilter> InClass)
{
	if (InClass == nullptr)
	{
		return;
	}

	// Skip SKEL and REINST classes.
	if (InClass->GetName().StartsWith(TEXT("SKEL_")) || InClass->GetName().StartsWith(TEXT("REINST_")))
	{
		return;
	}

	if (InClass->HasAnyClassFlags(CLASS_Abstract | CLASS_HideDropDown | CLASS_Deprecated) && InClass != UEmptySourceFilter::StaticClass())
	{
		return;
	}

	UClass* Class = InClass.Get();
	TRACE_TYPE(Class);

	DataSourceFilterClasses.Add(Class->GetName(), Class);

	const uint64 Identifier = TRACE_FILTER_IDENTIFIER(Class);
	if (!FilterClassIds.Contains(Class))
	{
		FilterClassIds.Add(Class, Identifier);
		IDToFilterClass.Add(Identifier, Class);

		UE_TRACE_LOG(SourceFilters, FilterClass, TraceSourceFiltersChannel)
			<< FilterClass.ClassId(Identifier);
	}
}

void FSourceFilterTrace::OutputInstance(const UDataSourceFilter* InFilter)
{
	if (!FilterInstances.Contains(InFilter))
	{
		TRACE_FILTER_CLASS(InFilter->GetClass());

		const uint64 ClassId = TRACE_FILTER_IDENTIFIER(InFilter->GetClass());
		const uint64 InstanceId = TRACE_FILTER_IDENTIFIER(InFilter);
		const uint64 SetId = 0;

		FilterInstances.Add(InFilter);
		IDToFilter.Add(InstanceId, InFilter);

		FText DisplayText;
		InFilter->Execute_GetDisplayText(InFilter, DisplayText);
		FString DisplayString = DisplayText.ToString();
		
		UE_TRACE_LOG(SourceFilters, FilterInstance, TraceSourceFiltersChannel)
			<< FilterInstance.ClassId(ClassId)
			<< FilterInstance.InstanceId(InstanceId)
			<< FilterInstance.SetId(SetId)
			<< FilterInstance.DisplayString(*DisplayString, DisplayString.Len());
	}
}

void FSourceFilterTrace::OutputSet(const UDataSourceFilterSet* InFilterSet)
{
	if (InFilterSet)
	{
		const uint64 SetId = TRACE_FILTER_IDENTIFIER(InFilterSet);

		IDToFilter.Add(SetId, InFilterSet);

		UE_TRACE_LOG(SourceFilters, FilterSet, TraceSourceFiltersChannel)
			<< FilterSet.SetId(SetId)
			<< FilterSet.Mode((uint8)InFilterSet->GetFilterSetMode());
	}
}

void FSourceFilterTrace::OutputFilterOperation(const UDataSourceFilter* InFilter, ESourceActorFilterOperation Operation, uint64 Parameter)
{
	const uint64 InstanceId = TRACE_FILTER_IDENTIFIER(InFilter);
	UE_TRACE_LOG(SourceFilters, FilterOperation, TraceSourceFiltersChannel)
		<< FilterOperation.InstanceId(InstanceId)
		<< FilterOperation.Operation((uint8)Operation)
		<< FilterOperation.Parameter(Parameter);
}

void FSourceFilterTrace::OutputFilterSettingsValue(const FString& InPropertyName, const uint8 InValue)
{
	UE_TRACE_LOG(SourceFilters, SetFilterSettingValue, TraceSourceFiltersChannel)
		<< SetFilterSettingValue.Value(InValue)
		<< SetFilterSettingValue.PropertyName(*InPropertyName, InPropertyName.Len());
}

void FSourceFilterTrace::OutputWorld(const UWorld* InWorld)
{
	const uint64 InstanceId = TRACE_FILTER_IDENTIFIER(InWorld);
	
	if (!IDsToWorldInstance.Contains(InstanceId))
	{
		IDsToWorldInstance.Add(InstanceId, InWorld);

		FString SendName;
		FTraceWorldFiltering::GetWorldDisplayString(InWorld, SendName);

		const uint32 AttachmentSize = (SendName.Len() + 1) * sizeof(TCHAR);
		auto WriteAttachment = [SendName, AttachmentSize](uint8* Out)
		{
			TCHAR* StringPtr = (TCHAR*)Out;
			FMemory::Memcpy(StringPtr, *SendName, AttachmentSize);
			StringPtr[AttachmentSize / sizeof(TCHAR) - 1] = '\0';
		};

		UE_TRACE_LOG(WorldSourceFilters, WorldInstance, TraceSourceFiltersChannel)
			<< WorldInstance.InstanceId(InstanceId)
			<< WorldInstance.Type((uint8)InWorld->WorldType)
			<< WorldInstance.State(CAN_TRACE_OBJECT(InWorld) ? 1 : 0)
			<< WorldInstance.SendName(*SendName, SendName.Len());
	}
}

void FSourceFilterTrace::OutputWorldOperation(const UWorld* InWorld, EWorldFilterOperation Operation, uint32 Parameter)
{
	const uint64 InstanceId = TRACE_FILTER_IDENTIFIER(InWorld);
	UE_TRACE_LOG(WorldSourceFilters, WorldOperation, TraceSourceFiltersChannel)
		<< WorldOperation.InstanceId(InstanceId)
		<< WorldOperation.Operation((uint8)Operation)
		<< WorldOperation.Parameter(Parameter);
}

template<typename T>
T* RetrieveTypeById(uint64 Id, const TMap<uint64, FObjectKey>& Map)
{
	T* Object = nullptr;
	if (const FObjectKey* ObjectKey = Map.Find(Id))
	{
		Object = Cast<T>(ObjectKey->ResolveObjectPtr());
	}

	return Object;
}

UClass* FSourceFilterTrace::RetrieveClassById(uint64 ClassId)
{
	return RetrieveTypeById<UClass>(ClassId, IDToFilterClass);
}

UClass* FSourceFilterTrace::RetrieveClassByName(const FString& ClassName)
{
	UClass* Class = nullptr;
	if (const FObjectKey* ObjectKeyPtr = DataSourceFilterClasses.Find(ClassName))
	{
		Class = Cast<UClass>(ObjectKeyPtr->ResolveObjectPtr());
	}
	return Class;
}

UDataSourceFilter* FSourceFilterTrace::RetrieveFilterbyId(uint64 FilterId)
{
	return RetrieveTypeById<UDataSourceFilter>(FilterId, IDToFilter);
}

UWorld* FSourceFilterTrace::RetrieveWorldById(uint64 WorldId)
{
	return RetrieveTypeById<UWorld>(WorldId, IDsToWorldInstance);
}

#endif // SOURCE_FILTER_TRACE_ENABLED

#undef LOCTEXT_NAMESPACE // "SourceFilterTrace"
