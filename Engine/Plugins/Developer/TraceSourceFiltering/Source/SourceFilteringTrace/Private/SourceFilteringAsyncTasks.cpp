// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceFilteringAsyncTasks.h"

#include "SourceFilterManager.h"

void FActorFilterAsyncTask::DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	QUICK_SCOPE_CYCLE_COUNTER(FActorFilterAsyncTask);
	if (bRunAsAsync)
	{
		check(!IsInGameThread() || IsRunningDedicatedServer());
		Manager->ApplyAsyncFilters();
	}
	else
	{
		check(IsInGameThread());
		Manager->ApplyGameThreadFilters();
	}
}

void FActorFilterApplyAsyncTask::DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	check(!IsInGameThread() || IsRunningDedicatedServer());
	if (IsValidChecked(Manager->World) && !Manager->World->IsUnreachable())
	{
		QUICK_SCOPE_CYCLE_COUNTER(FActorFilterApplyAsyncTask);
		Manager->ApplyFilterResults();
	}
}

void FActorFilterDrawStateAsyncTask::DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	check(IsInGameThread() || IsRunningDedicatedServer());
	if (IsValidChecked(Manager->World) && !Manager->World->IsUnreachable())
	{
		QUICK_SCOPE_CYCLE_COUNTER(FActorFilterDrawStateAsyncTask);
		Manager->DrawFilterResults();
	}
}
