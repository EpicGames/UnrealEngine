// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/TaskGraphInterfaces.h"

class FSourceFilterManager;

/** Async task executed on both GT and async thread to apply the corresponding type of filters */
class FActorFilterAsyncTask
{
	FSourceFilterManager* Manager;
	bool bRunAsAsync;
public:
	FActorFilterAsyncTask(FSourceFilterManager* InManager, bool bShouldRunAsAsync) : Manager(InManager), bRunAsAsync(bShouldRunAsAsync) {}

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FActorFilterAsyncTask, STATGROUP_TaskGraphTasks);
	}

	ENamedThreads::Type GetDesiredThread()
	{
		return bRunAsAsync ? ENamedThreads::AnyBackgroundHiPriTask : ENamedThreads::GameThread;
	}

	static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }
	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent);
};

/** Async task which handles applying the filtering result to the Engine-level trace filtering system (non-GT) */
class FActorFilterApplyAsyncTask
{
	FSourceFilterManager* Manager;
public:
	FActorFilterApplyAsyncTask(FSourceFilterManager* InManager) : Manager(InManager) {}

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FActorFilterApplyAsyncTask, STATGROUP_TaskGraphTasks);
	}

	ENamedThreads::Type GetDesiredThread()
	{
		return ENamedThreads::AnyBackgroundHiPriTask;
	}

	static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }
	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent);
};

/** Async task which handles drawing the filtering state on the GT */
class FActorFilterDrawStateAsyncTask
{
	FSourceFilterManager* Manager;

public:
	FActorFilterDrawStateAsyncTask(FSourceFilterManager* InManager) : Manager(InManager) {}

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FActorFilterDrawStateAsyncTask, STATGROUP_TaskGraphTasks);
	}

	ENamedThreads::Type GetDesiredThread()
	{
		return ENamedThreads::GameThread;
	}

	static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }
	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent);
};
