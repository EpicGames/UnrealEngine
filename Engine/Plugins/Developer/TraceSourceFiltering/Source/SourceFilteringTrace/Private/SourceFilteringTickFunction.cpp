// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceFilteringTickFunction.h"


#include "TraceSourceFilteringSettings.h"
#include "SourceFilterManager.h"

void FSourceFilteringTickFunction::ExecuteTick(float DeltaTime, ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	check(CurrentThread == ENamedThreads::GameThread);
	
	if (TickGroup == ETickingGroup::TG_PrePhysics)
	{
		Manager->SetupAsyncTasks(CurrentThread);
	}
	else if (TickGroup == ETickingGroup::TG_LastDemotable)
	{
		// In case the system is drawing the filtering state the GT should wait for this async task
		if (Manager->Settings->bDrawFilteringStates && IsValidRef(Manager->DrawTask))
		{
			MyCompletionGraphEvent->DontCompleteUntil(Manager->DrawTask);
		}
	}
}

FString FSourceFilteringTickFunction::DiagnosticMessage()
{
	return TEXT("FSourceFilteringTickFunction");
}
