// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineBaseTypes.h"

class FSourceFilterManager;

/** Tick function used for handling FSourceFilterManager ticking + kicking off async tasks */
struct FSourceFilteringTickFunction : public FTickFunction
{
	friend class FSourceFilterManager;

	FSourceFilteringTickFunction() : Manager(nullptr) {}
private:
	virtual void ExecuteTick(float DeltaTime, ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent) override;
	virtual FString DiagnosticMessage() override;
protected:
	FSourceFilterManager* Manager;
};
