// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceFilteringTraceModule.h"
#include "AssetRegistry/AssetData.h"
#include "Engine/Blueprint.h"
#include "Modules/ModuleManager.h"
#include "Misc/PackageName.h"
#include "UObject/UObjectIterator.h"
#include "Blueprint/BlueprintSupport.h"
#include "AssetRegistry/AssetRegistryModule.h"

#include "DataSourceFilterSet.h"
#include "SourceFilterTrace.h"
#include "TraceWorldFiltering.h"
#include "TraceSourceFiltering.h"

#if WITH_EDITOR
#include "Kismet2/KismetEditorUtilities.h"
#endif // WITH_EDITOR

void FSourceFilteringTraceModule::TraceFilterClasses()
{
	SCOPED_BOOT_TIMING("Trace_SourceFilterClasses");

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	TArray<FAssetData> Filters;
	TMultiMap<FName, FString> TagValues = { { FBlueprintTags::NativeParentClassPath, FObjectPropertyBase::GetExportPath(UDataSourceFilter::StaticClass()) } };
	AssetRegistryModule.Get().GetAssetsByTagValues(TagValues, Filters);

	for (const FAssetData& AssetData : Filters)
	{
		FAssetDataTagMapSharedView::FFindTagResult GeneratedClassPathPtr = AssetData.TagsAndValues.FindTag(TEXT("GeneratedClass"));
		if (GeneratedClassPathPtr.IsSet())
		{
			const FString BPClassObjectPath = FPackageName::ExportTextPathToObjectPath(GeneratedClassPathPtr.GetValue());
			const FString BPClassName = FPackageName::ObjectPathToObjectName(BPClassObjectPath);

			UClass* ChildClass = StaticLoadClass(UDataSourceFilter::StaticClass(), nullptr, *BPClassObjectPath);
			TRACE_FILTER_CLASS(ChildClass);
		}
	}

	for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
	{
		if (ClassIt->IsChildOf(UDataSourceFilter::StaticClass()))
		{
			if (UClass* FilterClass = *ClassIt)
			{
				if (FilterClass != UDataSourceFilter::StaticClass() && FilterClass != UDataSourceFilterSet::StaticClass())
				{
					TRACE_FILTER_CLASS(FilterClass);
				}
			}
		}
	}
}
#if WITH_EDITOR
void FSourceFilteringTraceModule::HandleNewFilterBlueprintCreated(UBlueprint* InBlueprint)
{
	TraceFilterClasses();
}
#endif // WITH_EDITOR

void FSourceFilteringTraceModule::StartupModule()
{
#if SOURCE_FILTER_TRACE_ENABLED
	FTraceWorldFiltering::Initialize();
	FTraceSourceFiltering::Initialize();

#if WITH_EDITOR
	// Add callback to trace out Filter Classes once the Asset Registry has finished loading 
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistryModule.Get().OnFilesLoaded().AddRaw(this, &FSourceFilteringTraceModule::TraceFilterClasses);

	// Add callback to trace out Filter Classes when a new UDataSourceFilter-based Blueprint has been created 
	FKismetEditorUtilities::RegisterOnBlueprintCreatedCallback(this, UDataSourceFilter::StaticClass(), FKismetEditorUtilities::FOnBlueprintCreated::CreateRaw(this, &FSourceFilteringTraceModule::HandleNewFilterBlueprintCreated));

	TraceFilterClasses();
#endif // WITH_EDITOR	
#endif // SOURCE_FILTER_TRACE_ENABLED
}

void FSourceFilteringTraceModule::ShutdownModule()
{
#if SOURCE_FILTER_TRACE_ENABLED

#if WITH_EDITOR
	FKismetEditorUtilities::UnregisterAutoBlueprintNodeCreation(this);
#endif // WITH_EDITOR
	FTraceWorldFiltering::Destroy();
#endif // SOURCE_FILTER_TRACE_ENABLED
}


IMPLEMENT_MODULE(FSourceFilteringTraceModule, SourceFilteringTrace);
