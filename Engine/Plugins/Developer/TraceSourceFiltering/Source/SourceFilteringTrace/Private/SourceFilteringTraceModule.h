// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"


class FSourceFilteringTraceModule : public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
protected:
	void TraceFilterClasses();
#if WITH_EDITOR	
	void HandleNewFilterBlueprintCreated(class UBlueprint* InBlueprint);
#endif // WITH_EDITOR
};
