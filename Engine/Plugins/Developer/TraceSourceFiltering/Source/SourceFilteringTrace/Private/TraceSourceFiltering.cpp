// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceSourceFiltering.h"
#include "DataSourceFilterSet.h"
#include "PropertyPathHelpers.h"
#include "SourceFilterCollection.h"
#include "TraceSourceFilteringSettings.h"
#include "SourceFilterTrace.h"
#include "TraceWorldFiltering.h"
#include "Misc/CommandLine.h"

#include "TraceSourceFilteringProjectSettings.h"
#include "UObject/Package.h"

FTraceSourceFiltering::FTraceSourceFiltering()
{
	LLM_SCOPE_BYNAME("Debug/Profiling");
	FilterCollection = NewObject<USourceFilterCollection>(GetTransientPackage(), NAME_None, RF_Transactional | RF_Transient);
	Settings = DuplicateObject(GetDefault<UTraceSourceFilteringSettings>(), nullptr);

	FString DefaultPresetPath;
	if (FParse::Value(FCommandLine::Get(), TEXT("-TraceSourcePreset="), DefaultPresetPath))
	{
		// Set default preset, if valid path
		FSoftObjectPath ObjectPath(DefaultPresetPath);
		if (USourceFilterCollection* Collection = Cast<USourceFilterCollection>(ObjectPath.TryLoad()))
		{
			FilterCollection->CopyData(Collection);
		}
	}
	// Try and load the user-defined default filter preset
	else if (USourceFilterCollection* PresetCollection = GetDefault<UTraceSourceFilteringProjectSettings>()->DefaultFilterPreset.LoadSynchronous())
	{	
		FilterCollection->CopyData(PresetCollection);	
	}

	PopulateRemoteTraceCommands();
}

void FTraceSourceFiltering::Initialize()
{
	FTraceSourceFiltering::Get();
}

FTraceSourceFiltering& FTraceSourceFiltering::Get()
{
	static FTraceSourceFiltering SourceFiltering;
	return SourceFiltering;
}

USourceFilterCollection* FTraceSourceFiltering::GetFilterCollection()
{
	return FilterCollection;
}

UTraceSourceFilteringSettings* FTraceSourceFiltering::GetSettings()
{
	return Settings;
}

#if SOURCE_FILTER_TRACE_ENABLED
static UClass* ConvertArgumentToClass(FString Argument)
{
	const uint64 ClassId = FCString::Atoi64(*Argument);

	UClass* Class = FSourceFilterTrace::RetrieveClassById(ClassId);
	checkf(Class != nullptr, TEXT("Failed to retrieve Class from command argument"));
	return Class;
}

static UDataSourceFilterSet* ConvertArgumentToFilterSet(FString Argument)
{
	const uint64 SetId = FCString::Atoi64(*Argument);
	// Cast as it is valid to specify a null Filter Set as a target for MoveFilter operation
	UDataSourceFilterSet* FilterSet = Cast<UDataSourceFilterSet>(FSourceFilterTrace::RetrieveFilterbyId(SetId));

	return FilterSet;
}

static EFilterSetMode ConvertArgumentToSetMode(FString Argument)
{
	const EFilterSetMode Mode = (EFilterSetMode)FCString::Atoi(*Argument);
	return Mode;
}

static UDataSourceFilter* ConvertArgumentToFilter(FString Argument)
{
	const uint64 InstanceId = FCString::Atoi64(*Argument);

	UDataSourceFilter* Filter = FSourceFilterTrace::RetrieveFilterbyId(InstanceId);
	ensure(Filter != nullptr);

	return Filter;
}

static UWorld* ConvertArgumentToWorld(FString Argument)
{
	const uint64 WorldId = FCString::Atoi64(*Argument);

	UWorld* World = FSourceFilterTrace::RetrieveWorldById(WorldId);
	ensure(World != nullptr);

	return World;
}
#endif // SOURCE_FILTER_TRACE_ENABLED

void FTraceSourceFiltering::ProcessRemoteCommand(const FString& Command, const TArray<FString>& Arguments)
{
	if (FFilterCommand* FilterCommand = CommandMap.Find(Command))
	{
		ensure(FilterCommand->NumExpectedArguments == Arguments.Num());
		FilterCommand->Function(Arguments);
	}
}

void FTraceSourceFiltering::AddReferencedObjects(FReferenceCollector& Collector)
{
	LLM_SCOPE_BYNAME("Debug/Profiling");
	Collector.AddReferencedObject(FilterCollection);
	Collector.AddReferencedObject(Settings);
}

void FTraceSourceFiltering::PopulateRemoteTraceCommands()
{
	LLM_SCOPE_BYNAME("Debug/Profiling");
#if SOURCE_FILTER_TRACE_ENABLED
CommandMap.Add(TEXT("AddFilterById"), { [this](const TArray<FString>& Arguments) -> void { FilterCollection->AddFilterOfClass(ConvertArgumentToClass(Arguments[0])); }, 1 });

	CommandMap.Add(TEXT("AddFilterClassToSet"),
		{ [this](const TArray<FString>& Arguments) -> void { FilterCollection->AddFilterOfClassToSet(ConvertArgumentToClass(Arguments[0]), ConvertArgumentToFilterSet(Arguments[1])); }, 2 });

	CommandMap.Add(TEXT("AddFilterToSet"),
		{ [this](const TArray<FString>& Arguments) -> void { FilterCollection->MoveFilter(ConvertArgumentToFilter(Arguments[0]), ConvertArgumentToFilterSet(Arguments[1])); }, 2 });

	CommandMap.Add(TEXT("MakeFilterSet"),
		{ [this](const TArray<FString>& Arguments) -> void { FilterCollection->MakeFilterSet(ConvertArgumentToFilter(Arguments[0]), ConvertArgumentToFilter(Arguments[1]), EFilterSetMode::AND); }, 2 });

	CommandMap.Add(TEXT("MakeSingleFilterSet"),
		{ [this](const TArray<FString>& Arguments) -> void { FilterCollection->ConvertFilterToSet(ConvertArgumentToFilter(Arguments[0]), ConvertArgumentToSetMode(Arguments[1])); }, 2 });

	CommandMap.Add(TEXT("ResetFilters"),
		{ [this](const TArray<FString>& Arguments) -> void { FilterCollection->Reset(); }, 0 });

	CommandMap.Add(TEXT("MoveFilter"),
		{ [this](const TArray<FString>& Arguments) -> void { FilterCollection->MoveFilter(ConvertArgumentToFilter(Arguments[0]), ConvertArgumentToFilterSet(Arguments[1])); }, 2 });

	CommandMap.Add(TEXT("RemoveFilter"),
		{ [this](const TArray<FString>& Arguments) -> void { FilterCollection->RemoveFilter(ConvertArgumentToFilter(Arguments[0])); }, 1 });

	CommandMap.Add(TEXT("SetFilterMode"),
		{ [](const TArray<FString>& Arguments) -> void
			{
				UDataSourceFilterSet* FilterSet = ConvertArgumentToFilterSet(Arguments[0]);
				EFilterSetMode Mode = ConvertArgumentToSetMode(Arguments[1]);
				FilterSet->SetFilterMode(Mode);
			},
			2
		}
	);

	CommandMap.Add(TEXT("SetFilterState"),
		{ [](const TArray<FString>& Arguments) -> void
			{
				UDataSourceFilter* Filter = ConvertArgumentToFilter(Arguments[0]);
				const bool bState = (bool)FCString::Atoi(*Arguments[1]);
				Filter->SetEnabled(bState);
			},
			2
		}
	);

	CommandMap.Add(TEXT("SetFilterSetting"),
		{ [this](const TArray<FString>& Arguments) -> void
			{
				FCachedPropertyPath PropertyPath(Arguments[0]);
				const bool bValue = (bool)FCString::Atoi(*Arguments[1]);
				ensure(PropertyPathHelpers::SetPropertyValue(Settings, PropertyPath, bValue));
				TRACE_FILTER_SETTINGS_VALUE(PropertyPath.ToString(), (uint8)bValue);
			},
			2
		}
	);

	CommandMap.Add(TEXT("SetFiltersFromPreset"),
		{ [this](const TArray<FString>& Arguments) -> void
			{
				FString PresetString = Arguments[0];

				// Parse out each class + parent index entry from the preset, each entry is stored as ClassName|ParentIndex
				TArray<FString> ClassIndexStrings;
				PresetString.ParseIntoArray(ClassIndexStrings, TEXT(";"));

				TArray<FString> ClassNames;
				TMap<int32, int32> ChildToParent;
				for (const FString& ClassIndexString : ClassIndexStrings)
				{
					// Separate out the class name from the index
					TArray<FString> Strings;
					ClassIndexString.ParseIntoArray(Strings, TEXT("|"));

					const FString ClassName = Strings[0];
					const int32 ParentIndex = FCString::Atoi(*Strings[1]);

					// Generate mapping from Child to Parent class instances
					const int32 FilterIndex = ClassNames.Add(ClassName);
					ChildToParent.Add(FilterIndex, ParentIndex);
				}

				FilterCollection->AddFiltersFromPreset(ClassNames, ChildToParent);
			},
			1
		}
	);

	CommandMap.Add(TEXT("SetWorldTypeFilterState"),
		{ [](const TArray<FString>& Arguments) -> void
			{
				const EWorldType::Type WorldType = (EWorldType::Type)FCString::Atoi(*Arguments[0]);
				const bool bState = (bool)FCString::Atoi(*Arguments[1]);

				FTraceWorldFiltering::SetStateByWorldType(WorldType, bState);
			},
			2
		}
	);

	CommandMap.Add(TEXT("SetWorldNetModeFilterState"),
		{ [](const TArray<FString>& Arguments) -> void
			{
				const ENetMode NetMode = (ENetMode)FCString::Atoi(*Arguments[0]);
				const bool bState = (bool)FCString::Atoi(*Arguments[1]);

				FTraceWorldFiltering::SetStateByWorldNetMode(NetMode, bState);
			},
			2
		}
	);

	CommandMap.Add(TEXT("SetWorldFilterState"),
		{ [](const TArray<FString>& Arguments) -> void
			{
				const UWorld* World = ConvertArgumentToWorld(Arguments[0]);
				const bool bState = (bool)FCString::Atoi(*Arguments[1]);
				FTraceWorldFiltering::SetWorldState(World, bState);
			},
			2
		}
	);
#endif // SOURCE_FILTER_TRACE_ENABLED
}
