// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceSourceFilteringTestBase.h"
#include "PreviewScene.h"
#include "SourceFilterManager.h"
#include "TraceFilter.h"
#include "TraceSourceFiltering.h"
#include "TraceWorldFiltering.h"

#if WITH_AUTOMATION_TESTS
bool FTraceSourceFilteringTestBase::RunTest(const FString& Parameters)
{
	// Reset the global frame number once the test has finished
	TGuardValue<uint32> FrameNumberGuard(GFrameNumber, GFrameNumber);
	Init();
	SetupTest(Parameters);
	TickFiltering();
	const bool bResult = CompareExpectedResults();
	Cleanup();
	return bResult;
}

void FTraceSourceFilteringTestBase::Init()
{
	FPreviewScene::ConstructionValues CV;
	CV.bTransactional = false;
	PreviewScene = new FPreviewScene(CV);

	World = TStrongObjectPtr<UWorld>(PreviewScene->GetWorld());
	Manager = const_cast<FSourceFilterManager*>(FTraceWorldFiltering::GetWorldSourceFilterManager(PreviewScene->GetWorld()));
	MARK_OBJECT_TRACEABLE(PreviewScene->GetWorld());

	FilterCollection = FTraceSourceFiltering::Get().GetFilterCollection();
	FilterCollection->Reset();
}

void FTraceSourceFilteringTestBase::Cleanup()
{
	Reset();

	World.Reset();
	Manager = nullptr;
	delete PreviewScene;

	FilterCollection = nullptr;
}

void FTraceSourceFilteringTestBase::Reset()
{
	FilterSets.Empty();
	Filters.Empty();

	FilterCollection->Reset();

	TArray<AActor*> SpawnedActors;
	ActorExpectedFilterResults.GenerateKeyArray(SpawnedActors);

	for (AActor* SpawnedActor : SpawnedActors)
	{
		World->RemoveActor(SpawnedActor, false);
	}

	ActorExpectedFilterResults.Empty();
}

FTraceSourceFilteringTestBase::FFilterSet& FTraceSourceFilteringTestBase::AddFilterSet(EFilterSetMode InMode)
{
	FilterSets.Add(new FFilterSet(FilterCollection, InMode));
	return FilterSets.Last();
}

void FTraceSourceFilteringTestBase::TickFiltering()
{
	for (uint32 TickIndex = 0; TickIndex <= NumTicks; ++TickIndex)
	{
		ManualTick();
	}
}

void FTraceSourceFilteringTestBase::ManualTick()
{
	Manager->ResetPerFrameData();

	if (FSourceFilterSetup::GetFilterSetup().HasGameThreadFilters())
	{
		Manager->ApplyGameThreadFilters();
	}

	if (FSourceFilterSetup::GetFilterSetup().HasAsyncFilters())
	{
		Manager->ApplyAsyncFilters();
	}

	if (FSourceFilterSetup::GetFilterSetup().RequiresApplyingFilters())
	{
		Manager->ApplyFilterResults();
	}

	// Increase frame number, interval filter depends on this
	++GFrameNumber;
}

bool FTraceSourceFilteringTestBase::CompareExpectedResults()
{
	for (const TPair<AActor*, bool>& ActorEntry : ActorExpectedFilterResults)
	{
		const bool bFilterResult = CAN_TRACE_OBJECT(ActorEntry.Key);
		UTEST_EQUAL("Actor filtering state", bFilterResult, ActorEntry.Value);
	}

	return true;
}


#endif // WITH_AUTOMATION_TESTS
