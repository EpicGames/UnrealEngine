// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceWorldFiltering.h"
#include "SourceFilterTrace.h"
#include "TraceFilters.h"

#include "ObjectTrace.h"
#include "SourceFilterManager.h"

#if WITH_EDITOR
#include "Editor.h"
#else
#include "UObject/Package.h"
#endif // WITH_EDITOR

FDelegateHandle FTraceWorldFiltering::WorldInitHandle;
FDelegateHandle FTraceWorldFiltering::WorldPostInitHandle;
FDelegateHandle FTraceWorldFiltering::WorldBeginTearDownHandle;
FDelegateHandle FTraceWorldFiltering::WorldCleanupHandle;
FDelegateHandle FTraceWorldFiltering::PreWorldFinishDestroyHandle;

FTraceWorldFiltering::FTraceWorldFilterStateChanged FTraceWorldFiltering::FilterStateChangedDelegate;

TArray<const UWorld*> FTraceWorldFiltering::Worlds;
TMap<EWorldType::Type, bool> FTraceWorldFiltering::WorldTypeFilterStates;
TMap<ENetMode, bool> FTraceWorldFiltering::NetModeFilterStates;

FCriticalSection FTraceWorldFiltering::WorldFilterStatesCritical;
TMap<const UWorld*, FSourceFilterManager*> FTraceWorldFiltering::WorldSourceFilterManagers;

#define LOCTEXT_NAMESPACE "FTraceWorldFilter" 

void FTraceWorldFiltering::Initialize()
{
	WorldInitHandle = FWorldDelegates::OnPreWorldInitialization.AddStatic(&FTraceWorldFiltering::OnWorldInit);
	WorldPostInitHandle = FWorldDelegates::OnPostWorldInitialization.AddStatic(&FTraceWorldFiltering::OnWorldPostInit);
	WorldCleanupHandle = FWorldDelegates::OnWorldCleanup.AddStatic(&FTraceWorldFiltering::OnWorldCleanup);
	WorldBeginTearDownHandle = FWorldDelegates::OnWorldBeginTearDown.AddStatic(&FTraceWorldFiltering::RemoveWorld);
	PreWorldFinishDestroyHandle = FWorldDelegates::OnPreWorldFinishDestroy.AddStatic(&FTraceWorldFiltering::RemoveWorld);

	// Disable engine level world filter
	DISABLE_ENGINE_WORLD_TRACE_FILTERING();
	DISABLE_ENGINE_ACTOR_TRACE_FILTERING();
}

void FTraceWorldFiltering::Destroy()
{
	FWorldDelegates::OnPreWorldInitialization.Remove(WorldInitHandle);
	FWorldDelegates::OnWorldCleanup.Remove(WorldCleanupHandle);
	FWorldDelegates::OnWorldBeginTearDown.Remove(WorldBeginTearDownHandle);
	FWorldDelegates::OnPreWorldFinishDestroy.Remove(PreWorldFinishDestroyHandle);

	for (const UWorld* World : Worlds)
	{
		if (World)
		{
			SetWorldState(World, true);
		}
	}
}

const FSourceFilterManager* FTraceWorldFiltering::GetWorldSourceFilterManager(const UWorld* World)
{
	return WorldSourceFilterManagers.FindRef(World);
}

const TArray<const UWorld*>& FTraceWorldFiltering::GetWorlds()
{
	return Worlds;
}

bool FTraceWorldFiltering::IsWorldTypeTraceable(EWorldType::Type InType)
{
	FScopeLock Lock(&WorldFilterStatesCritical);
	const bool* bStatePtr = WorldTypeFilterStates.Find(InType);
	return bStatePtr ? *bStatePtr : true;
}

bool FTraceWorldFiltering::IsWorldNetModeTraceable(ENetMode InNetMode)
{
	FScopeLock Lock(&WorldFilterStatesCritical);
	const bool* bStatePtr = NetModeFilterStates.Find(InNetMode);
	return bStatePtr ? *bStatePtr : true;
}

void FTraceWorldFiltering::SetStateByWorldType(EWorldType::Type WorldType, bool bState)
{
	// Update tracked flag 
	{
		FScopeLock Lock(&WorldFilterStatesCritical);
		WorldTypeFilterStates.FindOrAdd(WorldType) = bState;
	}
	UpdateWorldFiltering();

	// Trace out event for filtering change 
	uint32 Parameter = 0;
	Parameter |= ((uint32)WorldType << 16);
	Parameter |= (uint32)bState;

	OnFilterStateChanged().Broadcast();
	TRACE_WORLD_OPERATION(nullptr, EWorldFilterOperation::TypeFilter, Parameter);
}

void FTraceWorldFiltering::SetStateByWorldNetMode(ENetMode NetMode, bool bState)
{
	// Update tracked flag 
	{
		FScopeLock Lock(&WorldFilterStatesCritical);
		NetModeFilterStates.FindOrAdd(NetMode) = bState;
	}
	UpdateWorldFiltering();

	// Trace out event for filtering change 
	uint32 Parameter = 0;
	Parameter |= ((uint32)NetMode << 16);
	Parameter |= (uint32)bState;

	OnFilterStateChanged().Broadcast();
	TRACE_WORLD_OPERATION(nullptr, EWorldFilterOperation::NetModeFilter, Parameter);
}

void FTraceWorldFiltering::SetWorldState(const UWorld* InWorld, bool bState)
{
	// Mark specific UWorld as traceable
	SET_OBJECT_TRACEABLE(InWorld, bState);

	// Make sure to trace out contained ULevels when the provided World is marked as traceable
	if (bState)
	{
		for (FConstLevelIterator Iterator = InWorld->GetLevelIterator(); Iterator; ++Iterator)
		{
			TRACE_OBJECT(*Iterator);
		}
	}


	// Trace out event for world's filtering state 
	OnFilterStateChanged().Broadcast();
	TRACE_WORLD_OPERATION(InWorld, EWorldFilterOperation::InstanceFilter, bState);
}

void FTraceWorldFiltering::GetWorldDisplayString(const UWorld* InWorld, FString& OutDisplayString)
{
	const FString WorldName = InWorld->GetName();

	FString PostFix;
	if (InWorld->WorldType == EWorldType::PIE)
	{
		switch (InWorld->GetNetMode())
		{
		case NM_Client:
			PostFix = FString::Printf(TEXT("Client %i"), InWorld->GetOutermost()->GetPIEInstanceID() - 1);
			break;
		case NM_DedicatedServer:
		case NM_ListenServer:
			PostFix = LOCTEXT("ServerPostfix", "Server").ToString();
			break;
#if WITH_EDITOR
		case NM_Standalone:
			PostFix = GEditor->bIsSimulatingInEditor ? LOCTEXT("SimulateInEditorPostfix", "Simulate").ToString() : LOCTEXT("PlayInEditorPostfix", "PIE").ToString();
			break;
#endif // WITH_EDITOR
		}
	}
	else if (InWorld->WorldType == EWorldType::Editor || InWorld->WorldType == EWorldType::EditorPreview)
	{
		PostFix = LOCTEXT("EditorPostfix", "Editor").ToString();
	}
	else if (InWorld->WorldType == EWorldType::Game || InWorld->WorldType == EWorldType::GamePreview || InWorld->WorldType == EWorldType::GameRPC)
	{
		PostFix = LOCTEXT("GamePostfix", "Game").ToString();
	}

	OutDisplayString = FString::Printf(TEXT("%s [%s]"), *WorldName, *PostFix);
}

FTraceWorldFiltering::FTraceWorldFilterStateChanged& FTraceWorldFiltering::OnFilterStateChanged()
{
	return FilterStateChangedDelegate;
}

void FTraceWorldFiltering::OnWorldInit(UWorld* InWorld, const UWorld::InitializationValues IVS)
{
	if (InWorld && !Worlds.Contains(InWorld))
	{
		FScopeLock Lock(&FTraceWorldFiltering::WorldFilterStatesCritical);

		// Setup the initial filtering state according to set world properties
		const bool* bWorldTypeStatePtr = WorldTypeFilterStates.Find(InWorld->WorldType);
		const bool* bNetModeStatePtr = NetModeFilterStates.Find(InWorld->GetNetMode());

		const bool bTraceable = (bWorldTypeStatePtr ? *bWorldTypeStatePtr : true) && (bNetModeStatePtr ? *bNetModeStatePtr : true);
		SET_OBJECT_TRACEABLE(InWorld, bTraceable);

		// Create source filter manager for this world 
		FSourceFilterManager* Manager = new FSourceFilterManager(InWorld);
		WorldSourceFilterManagers.Add(InWorld, Manager);

		Worlds.Add(InWorld);

		OnFilterStateChanged().Broadcast();

		// Trace out new UWorld instance
		TRACE_WORLD_INSTANCE(InWorld);
	}
}

void FTraceWorldFiltering::OnWorldPostInit(UWorld* World, const UWorld::InitializationValues IVS)
{
	UpdateWorldFiltering();

	// Handle all loaded AActors within this world, to handle duplicated UWorlds (PIE)
	FSourceFilterManager* Manager = WorldSourceFilterManagers.FindChecked(World);
	Manager->OnFilterSetupChanged();
}

void FTraceWorldFiltering::OnWorldCleanup(UWorld* InWorld, bool bSessionEnded, bool bCleanupResources)
{
	RemoveWorld(InWorld);
}

void FTraceWorldFiltering::RemoveWorld(UWorld* InWorld)
{
	// Cleanup data for provided UWorld 
	if (Worlds.Remove(InWorld))
	{
		FSourceFilterManager* Manager = nullptr;
		if (WorldSourceFilterManagers.RemoveAndCopyValue(InWorld, Manager))
		{
			delete Manager;
		}

		TRACE_WORLD_OPERATION(InWorld, EWorldFilterOperation::RemoveWorld, 0);

		OnFilterStateChanged().Broadcast();
	}
}

void FTraceWorldFiltering::UpdateWorldFiltering()
{
	FScopeLock Lock(&FTraceWorldFiltering::WorldFilterStatesCritical);

	// Update all active worlds to new filtering state
	for (const UWorld* World : Worlds)
	{
		// Setup the initial filtering state according to set world properties
		const bool* bWorldTypeStatePtr = WorldTypeFilterStates.Find(World->WorldType);
		const bool* bNetModeStatePtr = NetModeFilterStates.Find(World->GetNetMode());

		const bool bTraceable = (bWorldTypeStatePtr ? *bWorldTypeStatePtr : true) && (bNetModeStatePtr ? *bNetModeStatePtr : true);
		SetWorldState(World, bTraceable);
	}
}

#undef LOCTEXT_NAMESPACE // "FTraceWorldFilter" 
