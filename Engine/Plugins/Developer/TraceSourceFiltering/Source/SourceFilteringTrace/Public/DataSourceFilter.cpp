// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataSourceFilter.h"
#include "DataSourceFiltering.h"
#include "SourceFilterTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataSourceFilter)

UDataSourceFilter::UDataSourceFilter() : bIsEnabled(true)
{

}

UDataSourceFilter::~UDataSourceFilter()
{

}

bool UDataSourceFilter::DoesActorPassFilter_Implementation(const AActor* InActor) const
{
	return DoesActorPassFilter_Internal(InActor);
}

void UDataSourceFilter::SetEnabled(bool bState)
{
	TRACE_FILTER_OPERATION(this, ESourceActorFilterOperation::SetFilterState, (uint64)bState);
	bIsEnabled = bState;
}

bool UDataSourceFilter::IsEnabled() const
{
	return bIsEnabled;
}

const FDataSourceFilterConfiguration& UDataSourceFilter::GetConfiguration() const
{
	return Configuration;
}

void UDataSourceFilter::GetDisplayText_Internal(FText& OutDisplayText) const
{
	if (UClass* Class = GetClass())
	{
		OutDisplayText = FText::FromString(GetClass()->GetName());
	}
}

bool UDataSourceFilter::DoesActorPassFilter_Internal(const AActor* InActor) const
{
	return true;
}

