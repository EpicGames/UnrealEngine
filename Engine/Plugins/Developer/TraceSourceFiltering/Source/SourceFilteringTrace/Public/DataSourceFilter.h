// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDataSourceFilterInterface.h"

#include "DataSourceFilter.generated.h"

#define UE_API SOURCEFILTERINGTRACE_API

UCLASS(MinimalAPI, Blueprintable)
class UDataSourceFilter : public UObject, public IDataSourceFilterInterface
{
	friend class FSourceFilterManager;
	friend class FSourceFilterSetup;

	GENERATED_BODY()
public:
	UE_API UDataSourceFilter();
	UE_API virtual ~UDataSourceFilter();

	UFUNCTION(BlueprintNativeEvent, Category = TraceSourceFiltering)
	UE_API bool DoesActorPassFilter(const AActor* InActor) const;
	UE_API virtual bool DoesActorPassFilter_Implementation(const AActor* InActor) const;

	/** Begin IDataSourceFilterInterface overrides */
	UE_API virtual void SetEnabled(bool bState) override;	
	UE_API virtual bool IsEnabled() const final;
	UE_API virtual const FDataSourceFilterConfiguration& GetConfiguration() const final;
protected:
	UE_API virtual void GetDisplayText_Internal(FText& OutDisplayText) const override;
	/** End IDataSourceFilterInterface overrides */

	UE_API virtual bool DoesActorPassFilter_Internal(const AActor* InActor) const;
protected:
	/** Filter specific settings */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = Filtering)
	FDataSourceFilterConfiguration Configuration;

	/** Whether or not this filter is enabled */
	bool bIsEnabled;
};

#undef UE_API
