// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataSourceFilter.h"
#include "IDataSourceFilterSetInterface.h"

#include "DataSourceFilterSet.generated.h"

#define UE_API SOURCEFILTERINGTRACE_API

/** Engine implementation of IDataSourceFilterSetInterface */
UCLASS(MinimalAPI, NotBlueprintable)
class UDataSourceFilterSet : public UDataSourceFilter, public IDataSourceFilterSetInterface
{
	friend class USourceFilterCollection;
	friend class FSourceFilterManager;
	friend class FSourceFilterSetup;

	GENERATED_BODY()
public:	
	const TArray<TObjectPtr<UDataSourceFilter>>& GetFilters() const { return Filters; }
	UE_API void SetFilterMode(EFilterSetMode InMode);

	/** Begin IDataSourceFilterSetInterface overrides */
	UE_API virtual EFilterSetMode GetFilterSetMode() const override;
	/** Begin IDataSourceFilterSetInterface overrides */

	/** Begin UDataSourceFilter overrides */
	UE_API virtual void SetEnabled(bool bState) override;
protected:
	UE_API virtual bool DoesActorPassFilter_Internal(const AActor* InActor) const override;
	UE_API virtual void GetDisplayText_Internal(FText& OutDisplayText) const override;
	/** End UDataSourceFilter overrides */

protected:
	/** Contained Filter instance */
	UPROPERTY()
	TArray<TObjectPtr<UDataSourceFilter>> Filters;

	/** Current Filter set operation */
	UPROPERTY()
	EFilterSetMode Mode;
};

#undef UE_API
