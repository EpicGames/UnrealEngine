// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataSourceFilter.h"

#include "EmptySourceFilter.generated.h"

/** Source filter implementation used to replace filter instance who's UClass is not loaded and or does not exist, primarily used by Filter Preset loading */
UCLASS(MinimalAPI, NotBlueprintable, hidedropdown)
class  UEmptySourceFilter : public UDataSourceFilter
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FString MissingClassName;

protected:
	virtual bool DoesActorPassFilter_Internal(const AActor* InActor) const override
	{
		return true;
	}

	virtual void GetDisplayText_Internal(FText& OutDisplayText) const override
	{
		OutDisplayText = FText::Format(NSLOCTEXT("UEmptySourceFilter", "DisplayText", "Empty Filter, missing original Filter Class ({0})"), FText::FromString(MissingClassName));
	}
};
