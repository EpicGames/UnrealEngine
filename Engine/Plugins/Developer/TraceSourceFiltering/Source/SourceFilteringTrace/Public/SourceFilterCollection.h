// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataSourceFilter.h"
#include "DataSourceFiltering.h"
#include "UObject/ObjectKey.h"
#include "Engine/DataAsset.h"

#include "SourceFilterCollection.generated.h"

#define UE_API SOURCEFILTERINGTRACE_API

class UDataSourceFilterSet;

UCLASS(MinimalAPI)
class USourceFilterCollection : public UDataAsset
{
	GENERATED_BODY()
public:
	virtual ~USourceFilterCollection() = default;

	/** Begin UDataAsset overrides */
	UE_API virtual void Serialize(FArchive& Ar) override;
	/** End UDataAsset overrides */

	/** Delegate which is broadcasted any of this collection's state changes */
	UE_API FSimpleMulticastDelegate& GetSourceFiltersUpdated();

	/** Add Filter instance to the collection, will be added at the root level */
	UE_API void AddFilter(UDataSourceFilter* NewFilter);

	/** Add a Filter instance of the provided class, will be added at the root level */
	UE_API UDataSourceFilter* AddFilterOfClass(const TSubclassOf<UDataSourceFilter>& FilterClass);
	/** Add a Filter instance of the provided class, added to the provided filter set */
	UE_API UDataSourceFilter* AddFilterOfClassToSet(const TSubclassOf<UDataSourceFilter>& FilterClass, UDataSourceFilterSet* FilterSet);

	/** Remove Filter instance, regardless of whether it is a root-level filter or part of a filter set  */
	UE_API void RemoveFilter(UDataSourceFilter* ToRemoveFilter);
	/** Remove a Filter Instance from a specific Filter Set */
	UE_API void RemoveFilterFromSet(UDataSourceFilter* ToRemoveFilter, UDataSourceFilterSet* FilterSet);
	
	/** Replace a Filter Instance with another */
	UE_API void ReplaceFilter(UDataSourceFilter* Destination, UDataSourceFilter* Source);
	
	/** Move a Filter instance to a specific Filter Set (moved to root-level if Destination = nullptr) */
	UE_API void MoveFilter(UDataSourceFilter* Filter, UDataSourceFilterSet* Destination);

	/** Sets whether or not a filter is enabled */
	UE_API void SetFilterState(UDataSourceFilter* Filter, bool bEnabledState);
	
	/** Convert a Filter Instance to a Filter Set (with provided mode), this creates set containing the replace filter */
	UE_API UDataSourceFilterSet* ConvertFilterToSet(UDataSourceFilter* ReplacedFilter, EFilterSetMode Mode);
	/** Create a Filter set (with provided mode) containing both Filter Instances */
	UE_API UDataSourceFilterSet* MakeFilterSet(UDataSourceFilter* FilterOne, UDataSourceFilter* FilterTwo, EFilterSetMode Mode);
	/** Set the filtering mode for the provided filter set*/
	UE_API void SetFilterSetMode(UDataSourceFilterSet* FilterSet, EFilterSetMode Mode);

	/** Creates an empty Filter Set (with provided mode) */
	UE_API UDataSourceFilterSet* MakeEmptyFilterSet(EFilterSetMode Mode);

	/** Creates a new collection of filter (sets), provided the filter class names and parent/child relationship */
	UE_API void AddFiltersFromPreset(const TArray<FString>& ClassNames, const TMap<int32, int32>& ChildToParentIndices);
	
	/** Resets all contained filter data */
	UE_API void Reset();
	
	/** Returns all top-level Filter instances */
	const TArray<UDataSourceFilter*>& GetFilters() const { return Filters; }

	/** Returns flattened Filter instances */
	UE_API void GetFlatFilters(TArray<TObjectPtr<UDataSourceFilter>>& OutFilters);

	/** Copies Filter data from other provided Filter Collection*/
	UE_API void CopyData(USourceFilterCollection* OtherCollection);

	/** Add a class filter, used to filter AActors on a high-level */
	UE_API void AddClassFilter(TSubclassOf<AActor> InClass);	
	UE_API void RemoveClassFilter(TSubclassOf<AActor> InClass);
	/** Returns all class filters  */
	const TArray<FActorClassFilter>& GetClassFilters() const { return ClassFilters; }
	/** Updating whether or not classes derived from the filter class should be included when applying filtering */
	UE_API void UpdateClassFilter(TSubclassOf<AActor> InClass, bool bIncludeDerivedClasses);

	/** Callback for patching up contained UDataSourceFilter blueprint instances which just got re-instanced */
	UE_API void OnObjectsReplaced(const TMap<UObject*, UObject*>& OldToNewInstanceMap);

	/** Returns parent filter set, if any, for provided filter */
	UE_API UDataSourceFilterSet* GetParentForFilter(UDataSourceFilter* Filter);
protected:
	/** Recursively removes filter and any contained child filters  */
	UE_API void RemoveFilterRecursive(UDataSourceFilter* ToRemoveFilter);

	UE_API void AddFilterToSet(UDataSourceFilter* Filter, UDataSourceFilterSet* FilterSet);

	/** Adds unique filter class name */
	UE_API void AddClassName(UDataSourceFilter* Filter);
	UE_API void RecursiveRetrieveFilterClassNames(UDataSourceFilter* Filter);
	UE_API void RecursiveGenerateFilterClassNames(UDataSourceFilter* Filter);
	UE_API UDataSourceFilter* RecursiveCopyFilter(UDataSourceFilter* Filter, int32& FilterOffset);
	
	template<typename T> T* CreateNewFilter(UClass* Class = T::StaticClass());
	UE_API void DestroyFilter(UDataSourceFilter* Filter);
protected:
	/** Root-level filter instances */
	UPROPERTY(VisibleAnywhere, Category=Filtering)
	TArray<TObjectPtr<UDataSourceFilter>> Filters;

	/** Class filters, used for high-level filtering of AActor instances inside of a UWorld */
	UPROPERTY(VisibleAnywhere, Category = Filtering)
	TArray<FActorClassFilter> ClassFilters;

	/** Mapping from Filter Instance FObjectKeys to their class names */
	TMap<FObjectKey, FString> FilterClassMap;

	/** Flat version of the Filter classes contained by this collection, stored according to Filters ordering, with child filters inline */
	UPROPERTY()
	TArray<FString> FilterClasses;

	/** Child / Parent mapping for Filter (sets) */
	UPROPERTY()
	TMap<TObjectPtr<UDataSourceFilter>, TObjectPtr<UDataSourceFilterSet>> ChildToParent;
	
	FSimpleMulticastDelegate SourceFiltersUpdatedDelegate;
};

#undef UE_API
