// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"

#define UE_API SOURCEFILTERINGTRACE_API

class UDataSourceFilter;
class UDataSourceFilterSet;
enum class ESourceActorFilterOperation;
enum class EWorldFilterOperation;
namespace UE::Trace { class FChannel; }
struct FObjectKey;
template <class TClass> class TSubclassOf;

#if UE_TRACE_ENABLED && !IS_PROGRAM && !UE_BUILD_SHIPPING
#define SOURCE_FILTER_TRACE_ENABLED 1
#else
#define SOURCE_FILTER_TRACE_ENABLED 0
#endif

#if SOURCE_FILTER_TRACE_ENABLED



UE_TRACE_CHANNEL_EXTERN(TraceSourceFiltersChannel)

struct FSourceFilterTrace
{
	/** Output trace data for a UDataSourceFilter (sub) class */
	static UE_API void OutputClass(const TSubclassOf<UDataSourceFilter> InClass);
	/** Output trace data for a UDataSourceFilter object instance */
	static UE_API void OutputInstance(const UDataSourceFilter* InFilter);
	/** Output trace data for a UDataSourceFilterSet object instance*/
	static UE_API void OutputSet(const UDataSourceFilterSet* InFilterSet);
	/** Output trace data for an operation involving a UDataSourceFilter(Set) instance */
	static UE_API void OutputFilterOperation(const UDataSourceFilter* InFilter, ESourceActorFilterOperation Operation, uint64 Parameter);

	/** Output trace data for a change in the UTraceSourceFilteringSettings for this running instance */
	static UE_API void OutputFilterSettingsValue(const FString& InPropertyName, const uint8 InValue);

	/** Output trace data for a UWorld's filtering related information */
	static UE_API void OutputWorld(const UWorld* InWorld);
	/** Output trace data for an operation involving a UWorld instance */
	static UE_API void OutputWorldOperation(const UWorld* InWorld, EWorldFilterOperation Operation, uint32 Parameter);	

	/** Tries to retrieve a UClass instance according to its Object Identifier */
	static UE_API UClass* RetrieveClassById(uint64 ClassId);
	/** Tries to retrieve a UClass instance according to its name */
	static UE_API UClass* RetrieveClassByName(const FString& ClassName);

	/** Tries to retrieve a UDataSourceFilter instance according to its Object Identifier */
	static UE_API UDataSourceFilter* RetrieveFilterbyId(uint64 FilterId);
	/** Tries to retrieve a UWorld instance according to its Object Identifier */
	static UE_API UWorld* RetrieveWorldById(uint64 WorldId);

protected:
	/** Mapping from a UClass's ObjectKey to their Object identifier retrieved from FObjectTrace::GetObjectId */
	static UE_API TMap<FObjectKey, uint64> FilterClassIds;
	/** Mapping from a UClass's Object identifier to an FObjectKey */
	static UE_API TMap<uint64, FObjectKey> IDToFilterClass;

	/** Mapping from a UDataSourceFilter's Object identifier to their FObjectKey */
	static UE_API TMap<uint64, FObjectKey> IDToFilter;

	/** Set of FObjectKey's, representing UDataSourceFilter instances, which have previously been traced out */
	static UE_API TSet<FObjectKey> FilterInstances;

	/** Mapping from an UClass's (sub class of UDataSourceFilter) name to their FObjectKey */
	static UE_API TMap<FString, FObjectKey> DataSourceFilterClasses;
	
	/** Mapping from an Object identifier (generated from a UWorld instance) to its FObjectKey */
	static UE_API TMap<uint64, FObjectKey> IDsToWorldInstance;
};

#define TRACE_FILTER_CLASS(Class) \
	FSourceFilterTrace::OutputClass(Class);

#define TRACE_FILTER_INSTANCE(Instance) \
	FSourceFilterTrace::OutputInstance(Instance);

#define TRACE_FILTER_SET(Set) \
	FSourceFilterTrace::OutputSet(Set);

#define TRACE_FILTER_OPERATION(Instance, Operation, Parameter) \
	FSourceFilterTrace::OutputFilterOperation(Instance, Operation, Parameter);

#define TRACE_FILTER_SETTINGS_VALUE(Name, Value) \
	FSourceFilterTrace::OutputFilterSettingsValue(Name, Value);

#define TRACE_WORLD_INSTANCE(World) \
	FSourceFilterTrace::OutputWorld(World);

#define TRACE_WORLD_OPERATION(Instance, Operation, Parameter) \
	FSourceFilterTrace::OutputWorldOperation(Instance, Operation, Parameter);

#define TRACE_FILTER_IDENTIFIER(Object) \
	FObjectTrace::GetObjectId(Object)
#else

#define TRACE_FILTER_CLASS(Class)
#define TRACE_FILTER_INSTANCE(Filter)
#define TRACE_FILTER_SET(Set)
#define TRACE_FILTER_OPERATION(Instance, Operation, Parameter)
#define TRACE_FILTER_SETTINGS_VALUE(Name, Value)

#define TRACE_WORLD_INSTANCE(World)
#define TRACE_WORLD_OPERATION(Instance, Operation, Parameter)

#define TRACE_FILTER_IDENTIFIER(Object) 0

#endif

#undef UE_API
