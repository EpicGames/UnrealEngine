// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/GCObject.h"

#define UE_API SOURCEFILTERINGTRACE_API


class UTraceSourceFilteringSettings;
class USourceFilterCollection;

/** Object managing the currently active UDataSourceFilter instances and UTraceSourceFilteringSettings */
class FTraceSourceFiltering : public FGCObject
{
public:
	static UE_API void Initialize();
	static UE_API FTraceSourceFiltering& Get();

	/** Returns the running instance its Filter Collection, containing active set of filters */
	UE_API USourceFilterCollection* GetFilterCollection();
	/** Returns the running instance its Filtering Settings */
	UE_API UTraceSourceFilteringSettings* GetSettings();

	/** Processes an received filtering command, altering the Filter Collection and or Settings accordingly */
	UE_API void ProcessRemoteCommand(const FString& Command, const TArray<FString>& Arguments);

	/** Begin FGCObject overrides */
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FTraceSourceFiltering");
	}
	/** End FGCObject overrides */
protected:
	UE_API FTraceSourceFiltering();
	UE_API void PopulateRemoteTraceCommands();

protected:
	TObjectPtr<UTraceSourceFilteringSettings> Settings;
	TObjectPtr<USourceFilterCollection> FilterCollection;

	/** Structure representing a remotely 'callable' filter command */
	struct FFilterCommand
	{
		TFunction<void(const TArray<FString>&)> Function;
		int32 NumExpectedArguments;
	};

	/** Mapping for all filtering commands from their name to respective FFilterCommand object */
	TMap<FString, FFilterCommand> CommandMap;
};

#undef UE_API
