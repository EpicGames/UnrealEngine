// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"


#include "TraceSourceFilteringProjectSettings.generated.h"

class UDataSourceFilter;
class USourceFilterCollection;

UCLASS(MinimalAPI, config = Engine, meta = (DisplayName = "Trace Source Filtering"), defaultconfig)
class UTraceSourceFilteringProjectSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UPROPERTY(config, EditAnywhere, Category = TraceSourceFiltering, AdvancedDisplay, meta = (DisplayName = "Source Filter Classes, which should be incorporated into the cook", RelativeToGameContentDir))
	TArray<TSoftClassPtr<UDataSourceFilter>> CookedSourceFilterClasses;

	UPROPERTY(config, EditAnywhere, Category = TraceSourceFiltering, AdvancedDisplay, meta = (DisplayName = "Default Filter preset, which should be loaded during boot", RelativeToGameContentDir))
	TSoftObjectPtr<USourceFilterCollection> DefaultFilterPreset;
};
