// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/AutomationTest.h"

#if WITH_AUTOMATION_TESTS
#include "Engine/World.h" // IWYU pragma: keep
#include "DataSourceFilterSet.h"
#include "SourceFilterCollection.h"
#include "UObject/StrongObjectPtr.h"

class FSourceFilterManager;
class FPreviewScene;
class UDataSourceFilter;


/** Base class for setting up functional Trace Source Filtering tests */
class FTraceSourceFilteringTestBase : public FAutomationTestBase
{
public:		
	FTraceSourceFilteringTestBase(const FString& InName, bool bIsComplex) : FAutomationTestBase(InName, bIsComplex), NumTicks(1) {}
	virtual ~FTraceSourceFilteringTestBase() = default;

	/** Begin FAutomationTestBase overrides */
	virtual EAutomationTestFlags GetTestFlags() const override { return EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter; }
	virtual bool IsStressTest() const {	return false; }
	virtual uint32 GetRequiredDeviceNum() const override { return 1; }
	SOURCEFILTERINGTRACE_API virtual bool RunTest(const FString& Parameters) override;
	/** End FAutomationTestBase overrides */

protected:
	/** Helper classes to make it easier to setup the test filter data */
	class IFilter
	{
	public:
		virtual ~IFilter() = default;
	};

	template<typename T>
	class FFilter : public IFilter
	{
	public:
		friend class FTraceSourceFilteringTestBase;

		T* operator->() const
		{
			return Filter;
		}

		T* operator*() const
		{
			return Filter;
		}

	protected:
		FFilter(T* InFilter) : Filter(InFilter)
		{

		}
		virtual ~FFilter() = default;

		T* Filter;
	};

	class FFilterSet
	{
	public:
		friend class FTraceSourceFilteringTestBase;

		template<typename T>
		FFilterSet& InsertFilter(FFilter<T>& Filter)
		{
			Collection->MoveFilter(*Filter, Set);
			return *this;
		}

		FFilterSet& InsertFilter(FFilterSet& FilterSet)
		{
			Collection->MoveFilter(FilterSet.Set, Set);
			return *this;
		}

	protected:
		FFilterSet(USourceFilterCollection* InCollection, EFilterSetMode InMode) : Collection(InCollection)
		{
			Set = Collection->MakeEmptyFilterSet(InMode);
		}

		/** Collection this set is part of */
		USourceFilterCollection* Collection;
		/** UObject representation for this set */
		UDataSourceFilterSet* Set;
		/** Contained filter sets */
		TArray<FFilterSet*> FilterSets;
		/** Contained filters */
		TArray<IFilter*> Filters;
	};

protected:
	/** Function used for setting up the test data, required to be implemented for a given test */
	virtual void SetupTest(const FString& Parameters) = 0;
	
	/** Adds a filter set, with the provided mode to the setup */
	SOURCEFILTERINGTRACE_API FFilterSet& AddFilterSet(EFilterSetMode InMode);

	/** Adds a filter for the provided class to the test filtering state */
	template<typename T>
	FFilter<T>& AddFilter()
	{
		ensure(T::StaticClass()->template IsChildOf<UDataSourceFilter>());
		T* FilterObj = CastChecked<T>(FilterCollection->AddFilterOfClass(T::StaticClass()));
		FFilter<T>* Filter = new FFilter<T>(FilterObj);
		Filters.Add(Filter);
		return *Filter;
	}

	/** Adds an actor and its expected filtering state (at the end of running the test) to the setup */
	template<typename T>
	T* AddActor(bool bExpectedFilteringState)
	{
		T* Actor = World->SpawnActor<T>();
		ActorExpectedFilterResults.Add(Actor, bExpectedFilteringState);

		return Actor;
	}

private:
	SOURCEFILTERINGTRACE_API void Init();
	SOURCEFILTERINGTRACE_API void TickFiltering();
	SOURCEFILTERINGTRACE_API void ManualTick();
	SOURCEFILTERINGTRACE_API bool CompareExpectedResults();
	SOURCEFILTERINGTRACE_API void Cleanup();
	SOURCEFILTERINGTRACE_API void Reset();
protected:
	/** Pocket world used for testing */
	TStrongObjectPtr<UWorld> World;
	/** Manager representing World's filtering state */
	FSourceFilterManager* Manager;
	/** Preview scene used to setup the pocket world */
	FPreviewScene* PreviewScene;
	/** Cached filter collection singleton ptr*/
	USourceFilterCollection* FilterCollection;

	/** Contained filter (sets) */
	TIndirectArray<FFilterSet> FilterSets;
	TIndirectArray<IFilter> Filters;
	
	/** Map from setup actor instances to its expected filtering state */
	TMap<AActor*, bool> ActorExpectedFilterResults;

	/** Total number of ticks this test is required to perform */
	uint32 NumTicks;
};

#endif // WITH_AUTOMATION_TESTS
