// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/World.h"

#define UE_API SOURCEFILTERINGTRACE_API

class FSourceFilterManager;

/** UWorld specific Trace filter, marks individual instances to not be traced out (and all containing actors / objects) */
struct FTraceWorldFiltering
{
	DECLARE_MULTICAST_DELEGATE(FTraceWorldFilterStateChanged);

	static UE_API void Initialize();
	static UE_API void Destroy();
	
	/** Retrieve a FSourceFilterManager instance representing the source filtering for a specific, provided) World instance */
	static UE_API const FSourceFilterManager* GetWorldSourceFilterManager(const UWorld* World);

	/** Return all Worlds currently tracked for Filtering */
	static UE_API const TArray<const UWorld*>& GetWorlds();

	/** Check whether or not a specific World Type can output Trace Data (not filtered out) */
	static UE_API bool IsWorldTypeTraceable(EWorldType::Type InType);
	/** Check whether or not a specific World's Net Mode can output Trace Data (not filtered out) */
	static UE_API bool IsWorldNetModeTraceable(ENetMode InNetMode);
	
	/** Set whether or not a specific World Type should be filtered out (or in) */
	static UE_API void SetStateByWorldType(EWorldType::Type WorldType, bool bState);	
	/** Set whether or not a specific World Net Mode should be filtered out (or in) */
	static UE_API void SetStateByWorldNetMode(ENetMode NetMode, bool bState);
	/** Set whether or not a specific UWorld instance's filtering state */
	static UE_API void SetWorldState(const UWorld* InWorld, bool bState);

	/** Returns a user facing display string for the provided UWorld instance */
	static UE_API void GetWorldDisplayString(const UWorld* InWorld, FString& OutDisplayString);

	/** Delegate which will be broadcast whenever the filtering state for any world (type, netmode) changes */
	static UE_API FTraceWorldFilterStateChanged& OnFilterStateChanged();

protected:
	/** Callbacks used to keep tracking of active (alive) UWorld instances */
	static UE_API void OnWorldInit(UWorld* World, const UWorld::InitializationValues IVS);
	static UE_API void OnWorldPostInit(UWorld* World, const UWorld::InitializationValues IVS);
	static UE_API void OnWorldCleanup(UWorld* World, bool bSessionEnded, bool bCleanupResources);
	static UE_API void RemoveWorld(UWorld* InWorld);

	static UE_API void UpdateWorldFiltering();

protected:
	static UE_API FDelegateHandle WorldInitHandle;
	static UE_API FDelegateHandle WorldPostInitHandle;
	static UE_API FDelegateHandle WorldBeginTearDownHandle;
	static UE_API FDelegateHandle WorldCleanupHandle;
	static UE_API FDelegateHandle PreWorldFinishDestroyHandle;

	/** Delegate for broadcasting filtering changes */
	static UE_API FTraceWorldFilterStateChanged FilterStateChangedDelegate;

	/** Array of currently active and alive UWorlds */
	static UE_API TArray<const UWorld*> Worlds;
	/** Per EWorldType enum entry flag, determines whether or not UWorld's of this type should be filtered out */
	static UE_API TMap<EWorldType::Type, bool> WorldTypeFilterStates;
	/** Per ENetMode enum entry flag, determines whether or not UWorld's using this netmode should be filtered out */
	static UE_API TMap<ENetMode, bool> NetModeFilterStates;
	/** Synchronization object for accessing WorldTypeFilterStates and NetModeFilterStates, required to ensure there is not competing access between Networking and Gamethread */
	static UE_API FCriticalSection WorldFilterStatesCritical;
	/** Mapping from UWorld instance to FSourceFilterManager, entries correspond to world instances in Worlds */
	static UE_API TMap<const UWorld*, FSourceFilterManager*> WorldSourceFilterManagers;
};

#undef UE_API
