// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SourceFilteringTrace : ModuleRules
{
	public SourceFilteringTrace(ReadOnlyTargetRules Target) : base(Target)
	{
        PrivateDependencyModuleNames.AddRange(
                new string[] {
                    "Core",
					"CoreUObject",
                    "TraceLog",
                    "Engine",
                    "SourceFilteringCore",
					"PropertyPath",
					"AssetRegistry",
					"DeveloperSettings"
				}
            );

        if (Target.Type == TargetType.Editor)
        {
            PrivateDependencyModuleNames.AddRange(
				new string[] {
					"UnrealEd"
				}
			);
        }
    }
}
