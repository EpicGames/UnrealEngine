{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Insights Data Source Filters",
	"Description": "Source data filtering for Unreal Insights.",
	"Category": "Insights",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "SourceFilteringTrace",
			"Type": "Runtime",
			"LoadingPhase": "PostEngineInit",
			"TargetConfigurationDenyList": [ "Shipping" ]
		},
		{
			"Name": "SourceFilteringCore",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"TargetConfigurationDenyList": [ "Shipping" ]
		},
		{
			"Name": "SourceFilteringEditor",
			"Type": "Editor",
			"LoadingPhase": "PreDefault"
		}
	],
	"Plugins": [
		{
			"Name": "GameplayInsights",
			"Enabled": true
		}
	]
}